// Copyright Epic Games, Inc. All Rights Reserved.

#include "SteamSocket.h"
#include "SteamSocketsSubsystem.h"
#include "SteamSocketsPrivate.h"
#include <steam/steamclientpublic.h>

FSteamSocket::FSteamSocket(ESocketType InSocketType, const FString& InSocketDescription, const FName& InSocketProtocol) :
	FSocket(InSocketType, InSocketDescription, InSocketProtocol),
	InternalHandle(k_HSteamNetConnection_Invalid),
	SendMode(k_nSteamNetworkingSend_UnreliableNoNagle),
	bShouldLingerOnClose(false),
	bIsListenSocket(false), 
	bIsLANSocket(false),
	bHasPendingData(false),
	ClosureReason(k_ESteamNetConnectionEnd_App_Generic)
{
	SocketSubsystem = static_cast<FSteamSocketsSubsystem*>(ISocketSubsystem::Get(STEAM_SOCKETS_SUBSYSTEM));
	ISteamNetworkingSockets* SocketInterface = FSteamSocketsSubsystem::GetSteamSocketsInterface();
	PollGroup = k_HSteamNetPollGroup_Invalid;
}

FSteamSocket::~FSteamSocket()
{
	// Release any data that is still around.
	if (bHasPendingData && PendingData != nullptr)
	{
		PendingData->Release();
	}

	ISteamNetworkingSockets* SocketInterface = FSteamSocketsSubsystem::GetSteamSocketsInterface();
	if (PollGroup != k_HSteamNetPollGroup_Invalid)
	{
		SocketInterface->DestroyPollGroup(PollGroup);
	}

	Close();
}

bool FSteamSocket::Close()
{
	bool bWasSuccessful = false;

	STEAM_SDK_IGNORE_REDUNDANCY_START
	// If we're already closed, don't bother doing anything else.
	if (InternalHandle == k_HSteamListenSocket_Invalid || InternalHandle == k_HSteamNetConnection_Invalid)
	{
		UE_LOGF(LogSockets, VeryVerbose, "SteamSockets: Socket is already cleaned up, ready for destruction.");
		return true;
	}
	STEAM_SDK_IGNORE_REDUNDANCY_END

	ISteamNetworkingSockets* SocketInterface = FSteamSocketsSubsystem::GetSteamSocketsInterface();

	// Safety check for the interface.
	if (SocketInterface == nullptr)
	{
		UE_LOGF(LogSockets, VeryVerbose, "SteamSockets: Socket interface not found, cannot close.");
		return false;
	}

	if (bIsListenSocket)
	{
		// Closing a listen socket destroys any sockets that were accepted from this socket
		bWasSuccessful = SocketInterface->CloseListenSocket(InternalHandle);
		if (bWasSuccessful)
		{
			UE_LOGF(LogSockets, Verbose, "SteamSockets: Closed listen socket %u via the API", InternalHandle);
			SocketSubsystem->RemoveSocketsForListener(this);
			InternalHandle = k_HSteamListenSocket_Invalid;
		}
	}
	else
	{
		bWasSuccessful = SocketInterface->CloseConnection(InternalHandle, (int32)ClosureReason, "Connection Ended.", bShouldLingerOnClose);
		if (bWasSuccessful)
		{
			UE_LOGF(LogSockets, Verbose, "SteamSockets: Closed socket %u via the API, reason %d", InternalHandle, ClosureReason);
			SocketSubsystem->QueueRemoval(InternalHandle);			
			InternalHandle = k_HSteamNetConnection_Invalid;
		}
	}

	return bWasSuccessful;
}

bool FSteamSocket::Bind(const FInternetAddr& Addr)
{
	if (Addr.GetProtocolType() == FNetworkProtocolTypes::SteamSocketsIP || 
		Addr.GetProtocolType() == FNetworkProtocolTypes::SteamSocketsP2P)
	{
		// Because of the SteamAPI, we have to just hold onto the bind address
		// We can't actually do anything with it until later.
		BindAddress = *((FInternetAddrSteamSockets*)&Addr);
		return true;
	}

	return false;
}

bool FSteamSocket::Connect(const FInternetAddr& Addr)
{
	ISteamNetworkingSockets* SocketInterface = FSteamSocketsSubsystem::GetSteamSocketsInterface();

	// Check if the respective interface is valid
	if (SocketInterface == nullptr)
	{
		UE_LOGF(LogSockets, Warning, "SteamSockets: Socket Interface is null, cannot connect");
		return false;
	}

	if (!Addr.IsValid())
	{
		UE_LOGF(LogSockets, Warning, "SteamSockets: Connection address is not valid. Cannot connect!");
		SocketSubsystem->LastSocketError = SE_EINVAL;
		return false;
	}

	FInternetAddrSteamSockets SteamAddr = *((FInternetAddrSteamSockets*)&Addr);
	if (GetProtocol() == FNetworkProtocolTypes::SteamSocketsIP)
	{
		SteamNetworkingConfigValue_t LanOptions;
		LanOptions.m_eDataType = k_ESteamNetworkingConfig_Int32;
		LanOptions.m_eValue = k_ESteamNetworkingConfig_IP_AllowWithoutAuth;
		LanOptions.m_val.m_int32 = (int32)bIsLANSocket;
		InternalHandle = SocketInterface->ConnectByIPAddress(SteamAddr, 1, &LanOptions);

	}
	else if (GetProtocol() == FNetworkProtocolTypes::SteamSocketsP2P)
	{
		InternalHandle = SocketInterface->ConnectP2P(SteamAddr, SteamAddr.GetPort(), 0, nullptr);
	}

	if (InternalHandle != k_HSteamNetConnection_Invalid)
	{
		UE_LOGF(LogSockets, Verbose, "SteamSockets: Connection to %ls initiated", *Addr.ToString(false));
		SocketSubsystem->AddSocket(Addr, this);

		return true;
	}

	return false;
}

bool FSteamSocket::Listen(int32 MaxBacklog)
{
	ISteamNetworkingSockets* SocketInterface = FSteamSocketsSubsystem::GetSteamSocketsInterface();

	// Require the interface to be non-null
	if (SocketInterface  == nullptr)
	{
		UE_LOGF(LogSockets, Warning, "SteamSockets: Socket interface is null, cannot establish interface for listening!");
		return false;
	}

	bool bWasSuccessful = false;
	bIsListenSocket = true;
	if (GetProtocol() == FNetworkProtocolTypes::SteamSocketsIP)
	{
		SteamNetworkingConfigValue_t LanOptions;
		LanOptions.m_eDataType = k_ESteamNetworkingConfig_Int32;
		LanOptions.m_eValue = k_ESteamNetworkingConfig_IP_AllowWithoutAuth;
		LanOptions.m_val.m_int32 = (int32)bIsLANSocket;
		InternalHandle = SocketInterface->CreateListenSocketIP(BindAddress, 1, &LanOptions);
	}
	else
	{
		InternalHandle = SocketInterface->CreateListenSocketP2P(BindAddress.GetPlatformPort(), 0, nullptr);
	}

	if (InternalHandle != k_HSteamListenSocket_Invalid)
	{
		bWasSuccessful = true;
		SocketSubsystem->AddSocket(BindAddress, this);

		PollGroup = SocketInterface->CreatePollGroup();
		SocketInterface->SetConnectionPollGroup(InternalHandle, PollGroup);
	}

	return bWasSuccessful;
}

class FSocket* FSteamSocket::Accept(const FString& InSocketDescription)
{
	FSteamSocket* NewSocket = static_cast<FSteamSocket*>(SocketSubsystem->CreateSocket(FName("SteamClientSocket"), InSocketDescription, GetProtocol()));
	NewSocket->SendMode = SendMode;
	NewSocket->bShouldLingerOnClose = bShouldLingerOnClose;
	NewSocket->bIsLANSocket = bIsLANSocket;

	// Set virtual ports here to preserve channel information over the P2P network.
	if (BindAddress.GetProtocolType() == FNetworkProtocolTypes::SteamSocketsP2P)
	{
		NewSocket->BindAddress.SetPlatformPort(BindAddress.GetPlatformPort());
	}

	return NewSocket;
}

bool FSteamSocket::SendTo(const uint8* Data, int32 Count, int32& BytesSent, const FInternetAddr& Destination)
{
	FInternetAddrSteamSockets CurrentAddress;
	if (GetPeerAddress(CurrentAddress) && Destination == CurrentAddress)
	{
		return Send(Data, Count, BytesSent);
	}
	else if (SocketSubsystem)
	{
		FSteamSocketsSubsystem::FSteamSocketInformation* SocketInfo = SocketSubsystem->GetSocketInfo(Destination);
		// IsValid will also check the validity of the socket pointer as well
		if (SocketInfo != nullptr && SocketInfo->IsValid())
		{
			return SocketInfo->Socket->Send(Data, Count, BytesSent);
		}
	}

	return false;
}

bool FSteamSocket::Send(const uint8* Data, int32 Count, int32& BytesSent)
{
	BytesSent = 0;
	// Steam won't send packets if we are not marked as connected.
	if (InternalHandle != k_HSteamNetConnection_Invalid && GetConnectionState() == SCS_Connected)
	{
		// GetConnectionState will check the validity of the sockets interface for us.
		switch (FSteamSocketsSubsystem::GetSteamSocketsInterface()->SendMessageToConnection(InternalHandle, (void*)Data, Count, SendMode, nullptr))
		{
			case k_EResultOK:
				SocketSubsystem->LastSocketError = SE_NO_ERROR;
				BytesSent = Count;
				return true;
			case k_EResultInvalidParam:
				SocketSubsystem->LastSocketError = SE_EINVAL;
			break;
			case k_EResultInvalidState:
				SocketSubsystem->LastSocketError = SE_EBADF;
			break;
			case k_EResultNoConnection:
				SocketSubsystem->LastSocketError = SE_ENOTCONN;
			break;
			case k_EResultIgnored:
				SocketSubsystem->LastSocketError = SE_SYSNOTREADY;
			break;
			case k_EResultLimitExceeded:
				SocketSubsystem->LastSocketError = SE_EPROCLIM;
			break;
			default:
				SocketSubsystem->LastSocketError = SE_EFAULT;
			break;
		}
		BytesSent = -1;
	}

	return false;
}

bool FSteamSocket::Recv(uint8* Data, int32 BufferSize, int32& BytesRead, ESocketReceiveFlags::Type Flags)
{
	BytesRead = -1;
	SteamNetworkingMessage_t* Message;
	int32 MessagesRead;

	if (RecvRaw(Message, 1, MessagesRead, Flags))
	{
		if (MessagesRead >= 1)
		{
			if (BufferSize >= 0 && Message->GetSize() <= (uint32)BufferSize)
			{
				FMemory::Memcpy(Data, Message->GetData(), BytesRead);
				Message->Release();
			}
			else
			{
				BytesRead = -1;
				SocketSubsystem->LastSocketError = SE_EMSGSIZE;
				return false;
			}		
		}
		else
		{
			BytesRead = 0;
		}
		return true;		
	}

	return false;
}

bool FSteamSocket::RecvRaw(SteamNetworkingMessage_t*& Data, int32 MaxMessages, int32& MessagesRead, ESocketReceiveFlags::Type Flags)
{
	ISteamNetworkingSockets* SocketInterface = FSteamSocketsSubsystem::GetSteamSocketsInterface();
	if (SocketInterface == nullptr)
	{
		SocketSubsystem->LastSocketError = SE_SYSNOTREADY;
		return false;
	}

	// Connection doesn't exist, return false
	if (InternalHandle == k_HSteamNetConnection_Invalid || (bIsListenSocket && InternalHandle == k_HSteamListenSocket_Invalid))
	{
		SocketSubsystem->LastSocketError = SE_EINVAL;
		return false;
	}

	bool bIsPeeking = (Flags == ESocketReceiveFlags::Peek);
	if (bIsPeeking)
	{
		// If someone asks for more than 1 message, this will break the message queue.
		if (MaxMessages > 1)
		{
			UE_LOGF(LogSockets, Warning, "SteamSockets: Recv cannot peek more than 1 message on the queue. Was asked for %d", MaxMessages);
			MaxMessages = 1;
		}

		// Don't attempt to peek again if we already have pending data as that will mess up the message queue.
		if (bHasPendingData)
		{
			MessagesRead = 1;
			return true;
		}
	}
	else if (bHasPendingData)
	{
		// Set the pointer value
		Data = PendingData;
		// null whatever we are currently holding
		PendingData = nullptr;
		MessagesRead = 1;
		bHasPendingData = false;
		return true;
	}

	// At this point, we will have already written our pending data or we're getting a new one.
	MessagesRead = (bIsListenSocket) ? SocketInterface->ReceiveMessagesOnPollGroup(PollGroup, ((bIsPeeking) ? &PendingData : &Data), MaxMessages) :
		SocketInterface->ReceiveMessagesOnConnection(InternalHandle, ((bIsPeeking) ? &PendingData : &Data), MaxMessages);

	if (MessagesRead >= 1)
	{
		if (bIsPeeking)
		{
			bHasPendingData = true;
		}
		SocketSubsystem->LastSocketError = SE_NO_ERROR;
		return true;
	}
	else if (MessagesRead == 0)
	{
		bHasPendingData = false;
		PendingData = nullptr;
		return true;
	}
	else
	{
		UE_LOGF(LogSockets, Error, "SteamSockets: Recv Connection handle is marked as invalid! Is Listen Socket? %d", bIsListenSocket);
		MessagesRead = -1;
	}

	SocketSubsystem->LastSocketError = SE_EFAULT;
	return false;
}

bool FSteamSocket::HasPendingData(uint32& PendingDataSize)
{
	if (bHasPendingData)
	{
		if (PendingData != nullptr)
		{
			PendingDataSize = PendingData->GetSize();
			return true;
		}
		else
		{
			UE_LOGF(LogSockets, Warning, "SteamSockets: HasPendingData flag is raised but the pendingdata object is null!");
			bHasPendingData = false;
		}
	}

	int32 MessagesRead;
	PendingDataSize = 0;
	// We still have to pass a valid pointer reference, but this will be unwritten to and unused.
	SteamNetworkingMessage_t* FakeMessage = nullptr;

	if (RecvRaw(FakeMessage, 1, MessagesRead, ESocketReceiveFlags::Peek))
	{
		if (MessagesRead >= 1 && PendingData != nullptr)
		{
			PendingDataSize = PendingData->GetSize();
			return true;
		}
	}
	
	return false;
}

ESocketConnectionState FSteamSocket::GetConnectionState()
{
	// Early out if we're not connected.
	ISteamNetworkingSockets* SocketInterface = FSteamSocketsSubsystem::GetSteamSocketsInterface();
	if (InternalHandle == k_HSteamNetConnection_Invalid || SocketInterface == nullptr)
	{
		return SCS_NotConnected;
	}

	SteamNetConnectionRealTimeStatus_t QuickSocketData;
	int nLanes = 0;
	SteamNetConnectionRealTimeLaneStatus_t* pLanes = NULL;
	if (SocketInterface->GetConnectionRealTimeStatus(InternalHandle, &QuickSocketData, nLanes, pLanes))
	{
		switch (QuickSocketData.m_eState)
		{
		case k_ESteamNetworkingConnectionState_Connected:
			return SCS_Connected;
		case k_ESteamNetworkingConnectionState_None:
		case k_ESteamNetworkingConnectionState_ProblemDetectedLocally:
			return SCS_ConnectionError;
		default:
			return SCS_NotConnected;
		}
	}

	return SCS_NotConnected;
}

void FSteamSocket::SetSendMode(int32 NewSendMode)
{
	SendMode = NewSendMode;
}

bool FSteamSocket::GetPeerAddress(FInternetAddr& OutAddr)
{
	ISteamNetworkingSockets* SocketInterface = FSteamSocketsSubsystem::GetSteamSocketsInterface();
	FInternetAddrSteamSockets& SteamAddr = (FInternetAddrSteamSockets&)OutAddr;
	SteamNetConnectionInfo_t CurrentConnectionInfo;
	if (SocketInterface && SocketInterface->GetConnectionInfo(InternalHandle, &CurrentConnectionInfo))
	{
		if (CurrentConnectionInfo.m_identityRemote.IsInvalid())
		{
			// There is no way to check validity on the remote addr structure
			// So the next best thing is to check if it's all zeros.
			if (CurrentConnectionInfo.m_addrRemote.IsIPv6AllZeros())
			{
				return false;
			}

			SteamAddr = CurrentConnectionInfo.m_addrRemote;
		}
		else
		{
			SteamAddr = CurrentConnectionInfo.m_identityRemote;
			// Remember to set the platform port here.
			SteamAddr.SetPlatformPort(BindAddress.GetPlatformPort());
		}
		return true;
	}

	return false;
}

bool FSteamSocket::SetNoDelay(bool bIsNoDelay)
{
	// See https://partner.steamgames.com/doc/api/steamnetworkingtypes#message_sending_flags for what all these flags mean
	STEAM_SDK_IGNORE_REDUNDANCY_START
	if (bIsNoDelay)
	{
		if (SendMode == k_nSteamNetworkingSend_Unreliable || SendMode == k_nSteamNetworkingSend_NoNagle || SendMode == k_nSteamNetworkingSend_UnreliableNoNagle)
		{
			SendMode = k_nSteamNetworkingSend_UnreliableNoDelay;
		}
		else if (SendMode == k_nSteamNetworkingSend_Reliable)
		{
			SendMode = k_nSteamNetworkingSend_ReliableNoNagle;
		}
	}
	else
	{
		if (SendMode == k_nSteamNetworkingSend_NoDelay)
		{
			SendMode = k_nSteamNetworkingSend_Unreliable;
		}
		else if (SendMode == k_nSteamNetworkingSend_UnreliableNoDelay)
		{
			SendMode = k_nSteamNetworkingSend_UnreliableNoNagle;
		}
		else if (SendMode == k_nSteamNetworkingSend_ReliableNoNagle) // Not necessarily a NoDelay, but this is the closest thing.
		{
			SendMode = k_nSteamNetworkingSend_Reliable;
		}
	}
	STEAM_SDK_IGNORE_REDUNDANCY_END
	return true;
}

bool FSteamSocket::SetLinger(bool bShouldLinger, int32 Timeout)
{
	bShouldLingerOnClose = bShouldLinger;
	return true;
}

void FSteamSocket::SetLanOptions()
{
	// This only should get set if you're in a LAN environment. 
	// The Netdriver allows us to set a lan socket flag which we'll use later for delayed listens.
	if (SteamNetworkingUtils() && bIsLANSocket && InternalHandle != k_HSteamNetConnection_Invalid)
	{
		SteamNetworkingUtils()->SetConnectionConfigValueInt32(InternalHandle, k_ESteamNetworkingConfig_IP_AllowWithoutAuth, 1);
	}
}

bool FSteamSocket::SetSendBufferSize(int32 Size, int32& NewSize)
{
	bool bSuccess = false;
	if (SteamNetworkingUtils())
	{
		if (SteamNetworkingUtils()->SetConnectionConfigValueInt32(InternalHandle, k_ESteamNetworkingConfig_SendBufferSize, Size))
		{
			bSuccess = true;
		}

		// Get the buffer size if we succeed or fail anyways.
		size_t ConfigValueSize = sizeof(int32);
		SteamNetworkingUtils()->GetConfigValue(k_ESteamNetworkingConfig_SendBufferSize, k_ESteamNetworkingConfig_Connection, InternalHandle, nullptr, &NewSize, &ConfigValueSize);
	}
	else
	{
		UE_LOGF(LogSockets, Warning, "SteamSockets: Setting buffer size requires access to network utilities!");
		NewSize = -1;
	}

	return bSuccess;
}

bool FSteamSocket::SetReceiveBufferSize(int32 Size, int32& NewSize)
{
	// This is 1:1 with SendBufferSize on SteamSockets.
	return SetSendBufferSize(Size, NewSize);
}

bool FSteamSocket::Shutdown(ESocketShutdownMode Mode)
{
	// Unsupported.
	return false;
}

bool FSteamSocket::Wait(ESocketWaitConditions::Type Condition, FTimespan WaitTime)
{
	// Unsupported. Can't tell if ready to write or not.
	return false;
}

bool FSteamSocket::WaitForPendingConnection(bool& bHasPendingConnection, const FTimespan& WaitTime)
{
	// Unfortunately, this API has no ability to check if there's a pending connection ahead of time.
	return false;
}

bool FSteamSocket::SetReuseAddr(bool bAllowReuse)
{
	// Unsupported.
	return true;
}

bool FSteamSocket::SetRecvErr(bool bUseErrorQueue)
{
	// Unsupported.
	return true;
}

bool FSteamSocket::SetNonBlocking(bool bIsNonBlocking)
{
	// Unsupported. API does not block.
	return true;
}

bool FSteamSocket::SetBroadcast(bool bAllowBroadcast)
{
	// Unsupported.
	return false;
}

bool FSteamSocket::JoinMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress)
{
	// Unsupported.
	return false;
}

bool FSteamSocket::JoinMulticastGroup(const FInternetAddr& GroupAddress)
{
	// Unsupported.
	return false;
}

bool FSteamSocket::LeaveMulticastGroup(const FInternetAddr& GroupAddress)
{
	// Unsupported.
	return false;
}

bool FSteamSocket::LeaveMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress)
{
	// Unsupported.
	return false;
}

bool FSteamSocket::SetMulticastLoopback(bool bLoopback)
{
	// Unsupported.
	return false;
}

bool FSteamSocket::SetMulticastTtl(uint8 TimeToLive)
{
	// Unsupported.
	return false;
}

bool FSteamSocket::SetMulticastInterface(const FInternetAddr& InterfaceAddress)
{
	// Unsupported.
	return false;
}
