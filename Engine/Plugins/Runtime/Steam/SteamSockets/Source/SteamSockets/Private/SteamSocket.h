// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPAddressSteamSockets.h"
#include "Sockets.h"

struct FTimespan;

class FSteamSocket : public FSocket
{
PACKAGE_SCOPE:
	HSteamNetPollGroup PollGroup;
	HSteamNetConnection InternalHandle;
	// Binding address to determine the Steam Socket protocol we'll be using
	FInternetAddrSteamSockets BindAddress;
	int32 SendMode;
	bool bShouldLingerOnClose;
	// Flag to differentiate which calls can be performed on this socket
	bool bIsListenSocket;
	// Flag to make sure we set the weaker authentication (skips SDR validation) in a LAN environment.
	bool bIsLANSocket;
	// Easy flag to quickly determine if the pending data needs to be taken before expiration.
	bool bHasPendingData;

	// Steam Internal value used for shutting off a socket
	ESteamNetConnectionEnd ClosureReason;

	// This is used for peeking and looking at pending data
	// Data is already internally handled in a queue on the SteamAPI side, this will give us whatever is at the top
	// Only set if called with HasPendingData or Recv with a Peek flag.
	SteamNetworkingMessage_t* PendingData;

	class FSteamSocketsSubsystem* SocketSubsystem;

	// Makes sure that sockets are properly set up in LAN environments.
	void SetLanOptions();

public:
	// FSocket implementation
	FSteamSocket(ESocketType InSocketType, const FString& InSocketDescription, const FName& InSocketProtocol);
	virtual ~FSteamSocket();

	virtual bool Close() override;

	virtual bool Bind(const FInternetAddr& Addr) override;
	virtual bool Connect(const FInternetAddr& Addr) override;
	virtual bool Listen(int32 MaxBacklog) override;

	virtual class FSocket* Accept(const FString& InSocketDescription) override;
	virtual class FSocket* Accept(FInternetAddr& OutAddr, const FString& InSocketDescription) override { return nullptr; }

	virtual bool SendTo(const uint8* Data, int32 Count, int32& BytesSent, const FInternetAddr& Destination) override;

	virtual bool Send(const uint8* Data, int32 Count, int32& BytesSent) override;
	virtual bool Recv(uint8* Data, int32 BufferSize, int32& BytesRead, ESocketReceiveFlags::Type Flags = ESocketReceiveFlags::None) override;

	virtual bool HasPendingData(uint32& PendingDataSize) override;
	virtual ESocketConnectionState GetConnectionState() override;

	virtual void GetAddress(FInternetAddr& OutAddr) override { OutAddr = BindAddress; }
	virtual bool GetPeerAddress(FInternetAddr& OutAddr) override;

	virtual bool SetNoDelay(bool bIsNoDelay = true) override;
	virtual bool SetLinger(bool bShouldLinger = true, int32 Timeout = 0) override;

	virtual bool SetSendBufferSize(int32 Size, int32& NewSize) override;
	virtual bool SetReceiveBufferSize(int32 Size, int32& NewSize) override;

	virtual int32 GetPortNo() override
	{
		return BindAddress.GetPort();
	}

	/** Unsupported functions */
	virtual bool Shutdown(ESocketShutdownMode Mode) override;
	virtual bool Wait(ESocketWaitConditions::Type Condition, FTimespan WaitTime) override;
	virtual bool WaitForPendingConnection(bool& bHasPendingConnection, const FTimespan& WaitTime) override;
	virtual bool SetReuseAddr(bool bAllowReuse = true) override;
	virtual bool SetRecvErr(bool bUseErrorQueue = true) override;
	virtual bool SetNonBlocking(bool bIsNonBlocking = true) override;
	virtual bool SetBroadcast(bool bAllowBroadcast = true) override;
	virtual bool JoinMulticastGroup(const FInternetAddr& GroupAddress) override;
	virtual bool JoinMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress) override;
	virtual bool LeaveMulticastGroup(const FInternetAddr& GroupAddress) override;
	virtual bool LeaveMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress);
	virtual bool SetMulticastLoopback(bool bLoopback) override;
	virtual bool SetMulticastTtl(uint8 TimeToLive) override;
	virtual bool SetMulticastInterface(const FInternetAddr& InterfaceAddress) override;

	// ~FSocket implementation

	bool RecvRaw(SteamNetworkingMessage_t*& Data, int32 MaxMessages, int32& MessagesRead, ESocketReceiveFlags::Type Flags = ESocketReceiveFlags::None);

	void SetClosureReason(ESteamNetConnectionEnd NewClosureReason) { ClosureReason = NewClosureReason; }
	void SetSendMode(int32 NewSendMode);

};
