// Copyright Epic Games, Inc. All Rights Reserved.

#include "SteamSocketsModule.h"
#include "SteamSocketsSubsystem.h"
#include "Misc/ConfigCacheIni.h"
#include "OnlineSubsystemSteam.h"
#include "SteamSharedModule.h"
#include "SocketSubsystemModule.h"

IMPLEMENT_MODULE(FSteamSocketsModule, SteamSockets);

void FSteamSocketsModule::StartupModule()
{
	FOnlineSubsystemSteam* OnlineSteamSubsystem = static_cast<FOnlineSubsystemSteam*>(IOnlineSubsystem::Get(STEAM_SUBSYSTEM));
	FSteamSharedModule& SharedModule = FSteamSharedModule::Get();
	const bool bIsNotEditor = (IsRunningDedicatedServer() || IsRunningGame());
	const bool bSteamOSSEnabled = (OnlineSteamSubsystem && OnlineSteamSubsystem->IsEnabled());

	// Load the Steam modules before first call to API
	if (SharedModule.AreSteamDllsLoaded() && bIsNotEditor && bSteamOSSEnabled)
	{
		// Settings flags
		bool bOverrideSocketSubsystem = true;

		// Use this flag from the SteamNetworking configuration so that the IPNetDrivers can be used as needed.
		if (GConfig)
		{
			GConfig->GetBool(TEXT("OnlineSubsystemSteam"), TEXT("bUseSteamNetworking"), bOverrideSocketSubsystem, GEngineIni);
		}

		// Create and register our singleton factory with the main online subsystem for easy access
		FSteamSocketsSubsystem* SocketSubsystem = FSteamSocketsSubsystem::Create();
		FString Error;
		if (SocketSubsystem->Init(Error))
		{
			bEnabled = true;

			// Register our socket Subsystem
			FSocketSubsystemModule& SSS = FModuleManager::LoadModuleChecked<FSocketSubsystemModule>("Sockets");
			SSS.RegisterSocketSubsystem(STEAM_SOCKETS_SUBSYSTEM, SocketSubsystem, bOverrideSocketSubsystem);
		}
		else
		{
			UE_LOGF(LogSockets, Error, "SteamSockets: Could not initialize SteamSockets, got error: %ls", *Error);
			FSteamSocketsSubsystem::Destroy();
		}
	}
	else if (!bSteamOSSEnabled)
	{
		UE_LOGF(LogSockets, Log, "SteamSockets: Disabled due to no Steam OSS running.");
	}
	else if(bIsNotEditor)
	{
		UE_LOGF(LogSockets, Warning, "SteamSockets: Steam SDK %ls libraries not present at %ls or failed to load!", STEAM_SDK_VER, *SharedModule.GetSteamModulePath());
	}
	else
	{
		UE_LOGF(LogSockets, Log, "SteamSockets: Disabled for editor process.");
	}
}

void FSteamSocketsModule::ShutdownModule()
{
	FModuleManager& ModuleManager = FModuleManager::Get();

	if (ModuleManager.IsModuleLoaded("Sockets"))
	{
		FSocketSubsystemModule& SSS = FModuleManager::GetModuleChecked<FSocketSubsystemModule>("Sockets");
		SSS.UnregisterSocketSubsystem(STEAM_SOCKETS_SUBSYSTEM);
	}
	FSteamSocketsSubsystem::Destroy();
}
