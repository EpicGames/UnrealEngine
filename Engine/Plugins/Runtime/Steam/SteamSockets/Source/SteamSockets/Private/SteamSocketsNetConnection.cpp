// Copyright Epic Games, Inc. All Rights Reserved.

#include "SteamSocketsNetConnection.h"
#include "SteamSocketsPackage.h"
#include "SteamSocketsSubsystem.h"
#include "SteamSocketsNetDriver.h"
#include "SteamSocket.h"
#include "Net/DataChannel.h"
#include "PacketHandlers/StatelessConnectHandlerComponent.h"

// Steam has the max value as 512 * 1024 (assumed bytes), which is unusable to us because UNetConnection asserts on any value greater than 1024
// Because of this, we cannot pass the value we get from Valve.
//
// As this value may change in a future Steam iteration without any notice, this is just set to 1024.

#include UE_INLINE_GENERATED_CPP_BY_NAME(SteamSocketsNetConnection)
static const int32 MAX_PACKET = 1024;

void USteamSocketsNetConnection::CleanUp()
{
	if (GetDriver() != nullptr && GetDriver()->GetSocketSubsystem() != nullptr && ConnectionSocket != nullptr)
	{
		FSteamSocketsSubsystem* SocketSub = static_cast<FSteamSocketsSubsystem*>(GetDriver()->GetSocketSubsystem());
		if (SocketSub != nullptr)
		{
			SocketSub->QueueRemoval(ConnectionSocket->InternalHandle);
		}
	}

	UNetConnection::CleanUp();
}

void USteamSocketsNetConnection::InitBase(UNetDriver* InDriver, FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket, int32 InPacketOverhead)
{
	// There is no packet overhead on this protocol, but since the UNetConnection will assert without a value greater than 0, we give it 1.
	UNetConnection::InitBase(InDriver, InSocket, InURL, InState, 
		((InMaxPacket == 0) ? MAX_PACKET : InMaxPacket),
		((InPacketOverhead == 0) ? 1 : InPacketOverhead));

	// Keep a reference to the socket.
	ConnectionSocket = static_cast<FSteamSocket*>(InSocket);
}

void USteamSocketsNetConnection::InitRemoteConnection(UNetDriver* InDriver, FSocket* InSocket, const FURL& InURL, const FInternetAddr& InRemoteAddr, EConnectionState InState,
	int32 InMaxPacket, int32 InPacketOverhead)
{
	// Call base so we set up all of our settings
	InitBase(InDriver, InSocket, InURL, InState, InMaxPacket, InPacketOverhead);

	// Grab the connection address from the URL
	RemoteAddr = InRemoteAddr.Clone();

	InitSendBuffer();
	SetClientLoginState(EClientLoginState::LoggingIn);
	SetExpectedClientLoginMsgType(NMT_Hello);
}

void USteamSocketsNetConnection::InitLocalConnection(UNetDriver* InDriver, FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket, int32 InPacketOverhead)
{
	// Call base so we set up all of our settings
	InitBase(InDriver, InSocket, InURL, InState, InMaxPacket, InPacketOverhead);

	if (InDriver == nullptr || InDriver->GetSocketSubsystem() == nullptr)
	{
		UE_LOGF(LogNet, Error, "SteamSockets: InitLocalConnection: Unable to get SocketSubsystem");
		Close();
		return;
	}

	// Grab the connection address from the URL
	RemoteAddr = InDriver->GetSocketSubsystem()->GetAddressFromString(InURL.Host);
	if (!RemoteAddr.IsValid() || !RemoteAddr->IsValid())
	{
		UE_LOGF(LogNet, Error, "SteamSockets: Cannot create connection address from NetConnection URL");
		Close();
		return;
	}
	RemoteAddr->SetPort(InURL.Port);

	InitSendBuffer();
}

void USteamSocketsNetConnection::LowLevelSend(void* Data, int32 CountBits, FOutPacketTraits& Traits)
{
	// Grab the socket subsystem now, we'll use it later.
	FSteamSocketsSubsystem* SocketSub = nullptr;
	if (GetDriver() != nullptr && GetDriver()->GetSocketSubsystem() != nullptr)
	{
		SocketSub = static_cast<FSteamSocketsSubsystem*>(GetDriver()->GetSocketSubsystem());
	}

	if (ConnectionSocket != nullptr && SocketSub)
	{
		const uint8* SendData = reinterpret_cast<const uint8*>(Data);

		// Process any packet modifiers
		if (Handler.IsValid() && !Handler->GetRawSend())
		{
			const ProcessedPacket ProcessedData = Handler->Outgoing(reinterpret_cast<uint8*>(Data), CountBits, Traits);

			if (!ProcessedData.bError)
			{
				SendData = ProcessedData.Data;
				CountBits = ProcessedData.CountBits;
			}
			else
			{
				CountBits = 0;
			}
		}

		int32 BytesSent = 0;		
		int32 BytesToSend = FMath::DivideAndRoundUp(CountBits, 8);

		// Make sure that we're within the acceptable size range.
		if (BytesToSend > k_cbMaxSteamNetworkingSocketsMessageSizeSend)
		{
			UE_LOGF(LogNet, Error, "SteamSockets: LowLevelSend: Attempted to send %d bytes of data, which is over the %d maximum", 
				BytesToSend, k_cbMaxSteamNetworkingSocketsMessageSizeSend);
			SocketSub->LastSocketError = SE_EMSGSIZE;
			return;
		}
		
		if (BytesToSend > 0)
		{
			if (!ConnectionSocket->Send(SendData, BytesToSend, BytesSent))
			{
				UE_LOGF(LogNet, Warning, "SteamSockets: LowLevelSend: Could not send %d bytes of data got error %d", BytesToSend, (int32)SocketSub->GetLastErrorCode());
			}
		}
	}
	else if (SocketSub)
	{
		UE_LOGF(LogNet, Error, "SteamSockets: LowLevelSend: Attempted to send data over a netconnection that has no socket!");
		SocketSub->LastSocketError = SE_EPROCLIM;
	}
}

FString USteamSocketsNetConnection::LowLevelGetRemoteAddress(bool bAppendPort)
{
	// If we have a remote addr set already (should always have this set)
	if (RemoteAddr.IsValid() && RemoteAddr->IsValid())
	{
		return RemoteAddr->ToString(bAppendPort);
	}

	// Otherwise, attempt to pull it from the socket itself.
	FInternetAddrSteamSockets AddressData;
	if (ConnectionSocket != nullptr)
	{
		ConnectionSocket->GetPeerAddress(AddressData);
		if (AddressData.IsValid())
		{
			return AddressData.ToString(bAppendPort);
		}
	}
	return TEXT("Invalid");
}

FString USteamSocketsNetConnection::LowLevelDescribe()
{
	return FString::Printf(TEXT("Addr=%s"), *RemoteAddr->ToString(true));
}

void USteamSocketsNetConnection::HandleRecvMessage(void* InData, int32 SizeOfData, const FInternetAddr* InFormattedAddress)
{
	uint8* RecvData = reinterpret_cast<uint8*>(InData);
	if (RecvData == nullptr || InFormattedAddress == nullptr)
	{
		return;
	}

	if (bInConnectionlessHandshake)
	{
		USteamSocketsNetDriver* SteamNetDriver = static_cast<USteamSocketsNetDriver*>(Driver);
		if (SteamNetDriver && !SteamNetDriver->ArePacketHandlersDisabled() && SteamNetDriver->ConnectionlessHandler.IsValid() && SteamNetDriver->StatelessConnectComponent.IsValid())
		{
			bool bRestartedHandshake = false;
			bool bPassedChallenge = false;

			TSharedPtr<const FInternetAddr> SharedAddress = InFormattedAddress->Clone();
			const ProcessedPacket RawPacket = SteamNetDriver->ConnectionlessHandler->IncomingConnectionless(SharedAddress, RecvData, SizeOfData);
			TSharedPtr<StatelessConnectHandlerComponent> StatelessConnect = SteamNetDriver->StatelessConnectComponent.Pin();

			if (!RawPacket.bError && StatelessConnect->HasPassedChallenge(SharedAddress, bRestartedHandshake) && !bRestartedHandshake)
			{
				// Set the initial packet sequence from the handshake data
				if (StatelessConnect.IsValid())
				{
					int32 ServerSequence = 0;
					int32 ClientSequence = 0;

					StatelessConnect->GetChallengeSequence(ServerSequence, ClientSequence);

					InitSequence(ClientSequence, ServerSequence);
				}

				// Start handshaking now
				if (Handler.IsValid())
				{
					Handler->BeginHandshaking();
				}

				UE_LOGF(LogNet, Log, "SteamSockets: Connectionless handshake complete");
				bInConnectionlessHandshake = false;

				// Transfer net driver's stateless connection data to the net connection
				if(StatelessConnectComponent.IsValid())
				{
					StatelessConnectComponent.Pin()->SetDriver(SteamNetDriver);
				}
				
				// Reset the challenge data for the future
				if (StatelessConnect.IsValid())
				{
					StatelessConnect->ResetChallengeData();
				}
				
				SizeOfData = FMath::DivideAndRoundUp(RawPacket.CountBits, 8);
				if (SizeOfData > 0)
				{
					RecvData = RawPacket.Data;
				}
				else
				{
					return;
				}
			}
		}
	}

	UNetConnection::ReceivedRawPacket(RecvData, SizeOfData);
}
