// Copyright Epic Games, Inc. All Rights Reserved.

#include "SteamSocketsNetDriver.h"
#include "SteamSocketsModule.h"
#include "SteamSocket.h"
#include "SteamSocketsNetConnection.h"
#include "SteamSocketsSubsystem.h"
#include "Engine/World.h"
#include "Misc/CommandLine.h"
#include "PacketHandler.h"
#include <steam/isteamnetworkingsockets.h>
#include <steam/steamclientpublic.h>

#include UE_INLINE_GENERATED_CPP_BY_NAME(SteamSocketsNetDriver)

void USteamSocketsNetDriver::PostInitProperties()
{
	Super::PostInitProperties();
}

void USteamSocketsNetDriver::Shutdown()
{
	UE_LOGF(LogSockets, Verbose, "SteamSockets: Shutdown called on netdriver");
	FSteamSocketsSubsystem* SocketSub = static_cast<FSteamSocketsSubsystem*>(GetSocketSubsystem());
	if (SocketSub && Socket)
	{
		FSteamSocketsSubsystem::FSteamSocketInformation* SocketInfo = SocketSub->GetSocketInfo(Socket->InternalHandle);
		// Make sure to remove any netdriver information as we are already shutting down.
		if (SocketInfo)
		{
			SocketInfo->NetDriver = nullptr;
			SocketInfo->MarkForDeletion();
		}

		Socket = nullptr;
	}

	Super::Shutdown();
}

bool USteamSocketsNetDriver::IsAvailable() const
{
	const FSteamSocketsSubsystem* SocketSub = static_cast<const FSteamSocketsSubsystem*>(ISocketSubsystem::Get(STEAM_SOCKETS_SUBSYSTEM));
	// Check if we have client handles and we active.
	if (SocketSub && SocketSub->IsSteamInitialized() && FSteamSocketsModule::Get().IsSteamSocketsEnabled())
	{
		return true;
	}
	return false;
}

bool USteamSocketsNetDriver::InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error)
{
	// Initialize base class
	if (!UNetDriver::InitBase(bInitAsClient, InNotify, URL, bReuseAddressAndPort, Error))
	{
		UE_LOGF(LogNet, Warning, "SteamSockets: Failed to initialize the base UNetDriver");
		return false;
	}

	if (!IsAvailable())
	{
		UE_LOGF(LogNet, Warning, "SteamSockets: SteamSockets are not enabled, cannot use netdriver!");
		return false;
	}

	FName SocketAddressType = NAME_None;
	const bool bIsLAN = URL.HasOption(TEXT("bIsLanMatch"));
	const bool bForcedIP = (bIsLAN || URL.HasOption(TEXT("bPassthrough")));
	FSteamSocketsSubsystem* SocketSub = static_cast<FSteamSocketsSubsystem*>(GetSocketSubsystem());
	if (!SocketSub)
	{
		UE_LOGF(LogNet, Warning, "SteamSockets: Unable to get SocketSubsystem");
		return false;
	}

	// Check to see if we're going to be using a P2P address
	const bool bIsUsingSteamAddrs = (SocketSub->IsUsingRelayNetwork());

	// This case is very special to the Steam framework and should not be followed as a template for other protocols.
	// On Steam Sockets we need to be logged in as a dedicated server if we are hosted or just using the relay network.
	// Because of this, we need to delay our actions and inquire for a login with the Steam network.
	bIsDelayedNetworkAccess = !bInitAsClient &&
		bIsUsingSteamAddrs &&
		!SocketSub->IsLoggedInToSteam() && !bForcedIP;

	// Delayed access and clients that will be hosting using SteamIDs will skip over the address determination and just use the P2P network.
	if (bIsDelayedNetworkAccess || (!bInitAsClient && !IsRunningDedicatedServer() && bIsUsingSteamAddrs && !bForcedIP))
	{
		SocketAddressType = FNetworkProtocolTypes::SteamSocketsP2P;
	}
	else if (bForcedIP) // Support LAN Matches over Steam Sockets
	{
		SocketAddressType = FNetworkProtocolTypes::SteamSocketsIP;
	}
	else
	{
		// This is the typical behavior of a connection in a NetDriver.
		// Figure out what kind of socket we'll need by parsing the Host field.
		TSharedPtr<FInternetAddr> SocketAddressHost = SocketSub->GetAddressFromString(URL.Host);
		if (SocketAddressHost.IsValid() && SocketAddressHost->IsValid())
		{
			SocketAddressType = SocketAddressHost->GetProtocolType();
		}
	}

	// Create the socket itself
	FString SocketDescription = (bInitAsClient) ? TEXT("Unreal client (Steam Sockets)") : TEXT("Unreal server (Steam Sockets)");
	Socket = static_cast<FSteamSocket*>(SocketSub->CreateSocket(FName(TEXT("SteamSocket")), SocketDescription, SocketAddressType));
	Socket->SetNoDelay(true);
	
	// Set up LAN flags early so that we make sure that the handshake doesn't go south when we don't have
	// a stable connection to the Steam backend.
	if (bIsLAN)
	{
		Socket->bIsLANSocket = true;
	}

	// On Servers, there's a possibility that the SocketAddressType is None due to not having enough information to determine
	// the user's intent. Because of this, the SocketSubsystem will have determined a correct protocol for us to use. We need
	// to update our address type so that we can find a good binding address to use later.
	if (SocketAddressType.IsNone())
	{
		SocketAddressType = Socket->GetProtocol();
	}

	// Delayed server login causes us to have a valid bind address much later than an usual process.
	// Because of this, a normal flow should be skipped over and use a forced P2P address.
	if (!bIsDelayedNetworkAccess)
	{
		UE_LOGF(LogNet, Verbose, "SteamSockets: Looking for a binding address that matches protocol %ls", *SocketAddressType.ToString());

		// Set the binding address.
		TArray<TSharedRef<FInternetAddr>> BindAddresses = SocketSub->GetLocalBindAddresses();
		if (BindAddresses.Num() > 0)
		{
			// Attempt to find a binding address that matches protocol of the connection destination
			for (int32 i = 0; i < BindAddresses.Num(); ++i)
			{
				UE_LOGF(LogNet, Verbose, "SteamSockets: Looking at binding address %ls", *BindAddresses[i]->ToString(true));
				if (BindAddresses[i]->GetProtocolType() == SocketAddressType)
				{
					UE_LOGF(LogNet, Verbose, "SteamSockets: Picked this binding address.");
					LocalAddr = BindAddresses[i]->Clone();
					break;
				}
			}
		}

		// Check for binding address issues.
		if (!LocalAddr.IsValid())
		{
			UE_LOGF(LogNet, Error, "SteamSockets: Could not determine the binding address!");
			return false;
		}
	}
	else
	{
		// Create a blank address, this will get filled in later when we log in.
		LocalAddr = SocketSub->CreateInternetAddr();
	}

	// Set the port
	LocalAddr->SetPort(URL.Port);

	// This should never fail.
	if (Socket->Bind(*LocalAddr) == false)
	{
		// If this fails, we somehow created a binding address that we cannot use.
		UE_LOGF(LogNet, Error, "SteamSockets: Invalid binding address used to create socket");
		return false;
	}

	return true;
}

bool USteamSocketsNetDriver::InitConnect(FNetworkNotify* InNotify, const FURL& ConnectURL, FString& Error)
{
	if (!InitBase(true, InNotify, ConnectURL, false, Error))
	{
		UE_LOGF(LogNet, Warning, "SteamSockets: InitConnect failed while setting up base information");
		return false;
	}

	// Create the NetConnection and channels
	ServerConnection = NewObject<USteamSocketsNetConnection>(NetConnectionClass);
	ServerConnection->InitLocalConnection(this, Socket, ConnectURL, USOCK_Pending);
	CreateInitialClientChannels();

	// Set the sequence flags if we have no packet handlers
	if (ArePacketHandlersDisabled())
	{
		ServerConnection->InitSequence(4, 4);
	}

	// Grab the subsystem
	FSteamSocketsSubsystem* SteamSubsystem = static_cast<FSteamSocketsSubsystem*>(GetSocketSubsystem());
	check(SteamSubsystem);

	// Figure out where we are going.
	TSharedPtr<FInternetAddr> SocketAddressHost = SteamSubsystem->GetAddressFromString(ConnectURL.Host);
	if (!SocketAddressHost.IsValid() || !SocketAddressHost->IsValid())
	{
		UE_LOGF(LogNet, Error, "SteamSockets: Could not obtain the address to connect to, had input %ls", *ConnectURL.Host);
		return false;
	}

	SocketAddressHost->SetPort(ConnectURL.Port);

	// Attempt to connect to the server
	if (!Socket->Connect(*SocketAddressHost))
	{
		UE_LOGF(LogNet, Warning, "SteamSockets: Could not connect to address %ls, got error code %d", *SocketAddressHost->ToString(true), 
			(int32)SteamSubsystem->GetLastErrorCode());
		return false;
	}

	// Link connection information for later
	SteamSubsystem->LinkNetDriver(Socket, this);

	UE_LOGF(LogNet, Log, "Game client on port %i, rate %i", ConnectURL.Port, ServerConnection->CurrentNetSpeed);

	return true;
}

bool USteamSocketsNetDriver::InitListen(FNetworkNotify* InNotify, FURL& LocalURL, bool bReuseAddressAndPort, FString& Error)
{
	if (!InitBase(false, InNotify, LocalURL, bReuseAddressAndPort, Error))
	{
		UE_LOGF(LogNet, Warning, "SteamSockets: InitListen failed while setting up base information");
		return false;
	}

	InitConnectionlessHandler();

	// Tell the SteamAPI to actually start listening.
	// This will actually create the socket with the SDK
	if (!bIsDelayedNetworkAccess && Socket->Listen(0) == false)
	{
		UE_LOGF(LogNet, Warning, "SteamSockets: InitListen failed to start listening on the Steam Network.");
		Socket->SetClosureReason(k_ESteamNetConnectionEnd_Misc_SteamConnectivity);  // k_ESteamNetConnectionEnd_Misc_RelayConnectivity has been deprecated

		return false;
	}

	// Watch for connections that come in.
	FSteamSocketsSubsystem* SteamSubsystem = static_cast<FSteamSocketsSubsystem*>(GetSocketSubsystem());
	check(SteamSubsystem);

	if (!bIsDelayedNetworkAccess)
	{
		SteamSubsystem->LinkNetDriver(Socket, this);
	}
	else
	{
		SteamSubsystem->AddDelayedListener(Socket, this);
	}

	UE_LOGF(LogNet, Log, "%ls started listening on %d", *GetDescription(), LocalURL.Port);
	UE_CLOGF((!bIsDelayedNetworkAccess && !UE_BUILD_SHIPPING), LogNet, Log, "SteamSockets: Listening with handle %u", Socket->InternalHandle);
	return true;
}

void USteamSocketsNetDriver::TickDispatch(float DeltaTime)
{
	// Do this first to clean up dead connections so we don't waste time processing them.
	UNetDriver::TickDispatch(DeltaTime);

	bool bIsAServer = (ServerConnection == nullptr);
	FSteamSocketsSubsystem* SteamSubsystem = static_cast<FSteamSocketsSubsystem*>(GetSocketSubsystem());

	// Receive packets until we cannot any longer.
	while (!bIsDelayedNetworkAccess)
	{
		// This could be placed out of the loop, however if we thread API event calls, this will need to be checked on every recv
		if (Socket != nullptr)
		{

#if !UE_BUILD_SHIPPING
			// This code causes peek cycles to run in order to test if the functionality works properly.
			if (SteamSubsystem && SteamSubsystem->bShouldTestPeek)
			{
				uint32 PendingDataSize;
				if (Socket->HasPendingData(PendingDataSize))
				{
					UE_LOGF(LogNet, Verbose, "SteamSockets: handle %u has %u data pending on the socket", Socket->InternalHandle, PendingDataSize);
				}
			}
#endif
			int32 BytesRead = 0;
			SteamNetworkingMessage_t* Message;
			// Attempt to grab a message using the Socket
			if (Socket->RecvRaw(Message, 1, BytesRead) && BytesRead > 0 && Message != nullptr)
			{
				USteamSocketsNetConnection* ConnectionToHandleMessage = static_cast<USteamSocketsNetConnection*>((bIsAServer) ? 
					FindClientConnectionForHandle(Message->m_conn) : ToRawPtr(ServerConnection));

				// Grab sender information for the purposes of logging
				FInternetAddrSteamSockets MessageSender(Message->m_identityPeer);

				// Set the P2P channel information if we're not over IP (which will already have the right data set)
				if (MessageSender.GetProtocolType() != FNetworkProtocolTypes::SteamSocketsIP)
				{
					MessageSender.SetPort(Message->m_nChannel);
				}

				// Process the message for this connection.
				if (ConnectionToHandleMessage != nullptr)
				{
					UE_LOGF(LogNet, VeryVerbose, "SteamSockets: Received packet from %ls with size %d", *MessageSender.ToString(true), Message->GetSize());
					ConnectionToHandleMessage->HandleRecvMessage(Message->m_pData, Message->GetSize(), &MessageSender);
					Message->Release();
					continue;
				}
				else
				{
					UE_LOGF(LogNet, Warning, "SteamSockets: Could not find connection information for sender %ls (handle: %u)", *MessageSender.ToString(true), Message->m_conn);
					Message->Release();
					continue;
				}
			}
			else if (BytesRead == 0)
			{
				// We have no more messages, leave.
				UE_LOGF(LogNet, VeryVerbose, "SteamSockets: Exhausted message, exiting loop.");
				break;
			}
			else
			{
				// In theory, we should have no information what so ever about connections that failed.
				// The sender should have no information in here and this should only happen if our handle is null.

				// A later disconnection event message from the API should handle cleanup of this object.
				UE_CLOGF(SteamSubsystem, LogNet, Warning, "SteamSockets: Could not recv message, got error code %d", SteamSubsystem->GetLastErrorCode());
				break;
			}
		}
		else
		{
			// Silently exit if the socket is invalid.
			return;
		}
	}
}

void USteamSocketsNetDriver::LowLevelSend(TSharedPtr<const FInternetAddr> Address, void* Data, int32 CountBits, FOutPacketTraits& Traits)
{
	if (Address.IsValid() && Address->IsValid() && Socket != nullptr)
	{
		const uint8* SendData = reinterpret_cast<const uint8*>(Data);
		if (ConnectionlessHandler.IsValid())
		{
			const ProcessedPacket ProcessedData =
				ConnectionlessHandler->OutgoingConnectionless(Address, (uint8*)SendData, CountBits, Traits);

			if (!ProcessedData.bError)
			{
				SendData = ProcessedData.Data;
				CountBits = ProcessedData.CountBits;
			}
			else
			{
				CountBits = 0;
			}
		}

		if (CountBits > 0)
		{
			int32 BytesToSend = FMath::DivideAndRoundUp(CountBits, 8);
			int32 SentBytes = 0;

			// Our sendto will find the correct socket to send over.
			if (!Socket->SendTo(SendData, BytesToSend, SentBytes, *Address))
			{
				UE_LOGF(LogNet, Warning, "SteamSockets: LowLevelSend: Could not send %d data over socket to %ls!", BytesToSend, *Address->ToString(true));
			}
		}
	}
	else
	{
		UE_LOGF(LogNet, Warning, "SteamSockets: LowLevelSend: Address or Socket was invalid, could not send data.");
	}
}

void USteamSocketsNetDriver::LowLevelDestroy()
{
	UNetDriver::LowLevelDestroy();

	FSteamSocketsSubsystem* SteamSubsystem = static_cast<FSteamSocketsSubsystem*>(GetSocketSubsystem());
	if (Socket != nullptr && !HasAnyFlags(RF_ClassDefaultObject))
	{
		SteamSubsystem->QueueRemoval(Socket->InternalHandle);
		Socket = nullptr;

		UE_LOGF(LogNet, Log, "%ls shut down", *GetDescription());
	}
}

class ISocketSubsystem* USteamSocketsNetDriver::GetSocketSubsystem()
{
	return ISocketSubsystem::Get(STEAM_SOCKETS_SUBSYSTEM);
}

bool USteamSocketsNetDriver::IsNetResourceValid(void)
{
	return Socket != nullptr && (!ServerConnection || (ServerConnection && ServerConnection->GetConnectionState() == USOCK_Open));
}

bool USteamSocketsNetDriver::ArePacketHandlersDisabled() const
{
#if !UE_BUILD_SHIPPING
	return FParse::Param(FCommandLine::Get(), TEXT("NoPacketHandler"));
#else
	return false;
#endif
}

void USteamSocketsNetDriver::ResetSocketInfo(const FSteamSocket* RemovedSocket)
{
	const SteamSocketHandles SocketHandle = RemovedSocket->InternalHandle;
	USteamSocketsNetConnection* SocketConnection = 
		static_cast<USteamSocketsNetConnection*>(ServerConnection ? ToRawPtr(ServerConnection) : FindClientConnectionForHandle(SocketHandle));

	if (SocketConnection)
	{
		SocketConnection->ClearSocket();
	}

	// If this netdriver has the same socket pointer, go ahead and remove it.
	if (Socket == RemovedSocket)
	{
		Socket = nullptr;
	}
}

UNetConnection* USteamSocketsNetDriver::FindClientConnectionForHandle(SteamSocketHandles SocketHandle)
{
	for (TObjectPtr<UNetConnection>& ClientConnection : ClientConnections)
	{
		USteamSocketsNetConnection* SteamConnection = static_cast<USteamSocketsNetConnection*>(ClientConnection);
		if (SteamConnection && SteamConnection->GetRawSocket() != nullptr)
		{
			const FSteamSocket* SteamSocket = SteamConnection->GetRawSocket();
			if (SteamSocket && SteamSocket->InternalHandle == SocketHandle)
			{
				return ClientConnection;
			}
		}
	}
	return nullptr;
}

void USteamSocketsNetDriver::OnConnectionCreated(SteamSocketHandles ListenParentHandle, SteamSocketHandles SocketHandle)
{
	FSteamSocketsSubsystem* SocketSubsystem = static_cast<FSteamSocketsSubsystem*>(GetSocketSubsystem());

	// If this netdriver does not hold the listener for the incoming connection, do not handle it
	// This makes sure cleanup of listener socket later does not destroy sockets it does not own.
	// (useful for beacons)
	if (Socket == nullptr || ListenParentHandle != ((FSteamSocket*)Socket)->InternalHandle || !SocketSubsystem)
	{
		return;
	}

	// Absolutely make sure that we can interface with the SteamAPI
	check(FSteamSocketsSubsystem::GetSteamSocketsInterface());

	// Unlike the other SteamNetworking functionality, connections that we don't want cannot just be ignored
	// So instead of processing everything and then disconnecting unwanted connections, we drop them immediately.
	if (Notify != nullptr && Notify->NotifyAcceptingConnection() == EAcceptConnection::Accept)
	{
		// Accept the connection with the API. We'll want to do this as quick as possible.
		EResult AcceptedResult = FSteamSocketsSubsystem::GetSteamSocketsInterface()->AcceptConnection(SocketHandle);
		if (AcceptedResult != k_EResultOK)
		{
			// We can fail here because the client aborted or something happened to our connection to the network
			UE_LOGF(LogNet, Error, "SteamSockets: Cannot accept connection due to error %d", (int32)AcceptedResult);
			SocketSubsystem->LastSocketError = SE_ECONNRESET;
			return;
		}

		// Create the new connection
		USteamSocketsNetConnection* NewConnection = NewObject<USteamSocketsNetConnection>(NetConnectionClass);
		check(NewConnection);

		// Create the bookkeeping and set up the socket.
		FInternetAddrSteamSockets ConnectedAddr;
		FSteamSocket* NewSocket = static_cast<FSteamSocket*>(Socket->Accept(TEXT("AcceptedSocket")));
		NewSocket->InternalHandle = SocketHandle;

#if !PLATFORM_MAC
		// Hotfix-safe version, should get refactored into a separate function that also sets InternalHandle
		SocketSubsystem->GetSteamSocketsInterface()->SetConnectionPollGroup(SocketHandle, Socket->PollGroup);
#endif // PLATFORM_MAC

		NewSocket->GetPeerAddress(ConnectedAddr);
		SocketSubsystem->AddSocket(ConnectedAddr, NewSocket, Socket);
		SocketSubsystem->LinkNetDriver(NewSocket, this);

		EConnectionState RemoteConnectionState = USOCK_Pending;
		SteamNetConnectionRealTimeStatus_t ConnStatus;
		int nLanes = 0;
		SteamNetConnectionRealTimeLaneStatus_t* pLanes = NULL;
		// We might already have a full route to the user already, which will not signal an event later (for the listener) that would otherwise update the connection state
		// Because of this, we need to quickly query the new connection status to determine where we are in the connection flow
		// This is very unlikely unless we already had a previous route to this connection.
		if (FSteamSocketsSubsystem::GetSteamSocketsInterface()->GetConnectionRealTimeStatus(SocketHandle, &ConnStatus, nLanes, pLanes) && ConnStatus.m_eState == k_ESteamNetworkingConnectionState_Connected)
		{
			// We have already found a route and made a connection, skip out of pending.
			RemoteConnectionState = USOCK_Open;
		}

		// Create the net connection structures and add the connections.
		NewConnection->InitRemoteConnection(this, NewSocket, World ? World->URL : FURL(), ConnectedAddr, RemoteConnectionState);

		if (ArePacketHandlersDisabled())
		{
			// Set up the sequence numbers. Valve uses their own, but so do we so to prevent
			// reinventing the wheel, we'll set up our sequence numbers to be some random garbage
			NewConnection->InitSequence(4, 4);

			// Attempt to start the PacketHandler handshakes (we do not support stateless connect)
			if (NewConnection->Handler.IsValid())
			{
				NewConnection->Handler->BeginHandshaking();
			}
		}
		else if (ConnectionlessHandler.IsValid() && StatelessConnectComponent.IsValid())
		{
			NewConnection->FlagForHandshake();
		}

		Notify->NotifyAcceptedConnection(NewConnection);
		AddClientConnection(NewConnection);

		UE_LOGF(LogNet, Log, "SteamSockets: New connection (%u) over listening socket accepted %ls", SocketHandle, *ConnectedAddr.ToString(true));
	}
	else
	{
		FSteamSocketsSubsystem::GetSteamSocketsInterface()->CloseConnection(SocketHandle, k_ESteamNetConnectionEnd_App_Generic, "Connection rejected", false);
		UE_LOGF(LogNet, Log, "SteamSockets: New connection over listening socket rejected.");
	}
}

void USteamSocketsNetDriver::OnConnectionUpdated(SteamSocketHandles SocketHandle, int32 NewState)
{
	// While this function can also get the relay status flags, we really only
	// care about the connected state flag.
	if (NewState == k_ESteamNetworkingConnectionState_Connected)
	{
		UNetConnection* SocketConnection = (ServerConnection) ? ToRawPtr(ServerConnection) : FindClientConnectionForHandle(SocketHandle);
		if (SocketConnection)
		{
			SocketConnection->SetConnectionState(USOCK_Open);
		}

		UE_LOGF(LogNet, Verbose, "SteamSockets: Connection established with user with socket id: %u", SocketHandle);

		// For clients, we need to match the settings of the server
		if (ServerConnection && ArePacketHandlersDisabled())
		{
			// Attempt to start the PacketHandler handshakes (we do not support stateless connect)
			// The PendingNetGame will also do it but we can't actually send the packets for it until we're connected.
			if (ServerConnection->Handler.IsValid())
			{
				ServerConnection->Handler->BeginHandshaking();
			}
		}
	}
}

void USteamSocketsNetDriver::OnConnectionDisconnected(SteamSocketHandles SocketHandle)
{
	UNetConnection* SocketConnection = ServerConnection ? ToRawPtr(ServerConnection) : FindClientConnectionForHandle(SocketHandle);
	if (SocketConnection)
	{
		SocketConnection->SetConnectionState(USOCK_Closed);
	}

	UE_LOGF(LogNet, Verbose, "SteamSockets: Connection dropped with user with socket id: %u", SocketHandle);
}
