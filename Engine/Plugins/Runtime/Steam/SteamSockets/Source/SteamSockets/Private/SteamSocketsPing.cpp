// Copyright Epic Games, Inc. All Rights Reserved.

#include "SteamSocketsPing.h"
#include "EngineLogs.h"
#include "SteamSocketsSubsystem.h"
#include <steam/isteamnetworkingutils.h>
#include <steam/steamnetworkingtypes.h>
#include <steam/steamtypes.h>

bool FSteamSocketsPing::IsUsingP2PRelays() const
{
	if (SocketSub)
	{
		return SocketSub->IsUsingRelayNetwork();
	}

	return false;
}

FString FSteamSocketsPing::GetHostPingData() const
{
	// Don't do anything if we're not using the relay network!
	if (!IsUsingP2PRelays())
	{
		UE_LOGF(LogNet, Warning, "SteamSockets: GetHostPingData was called but relays are disabled! This is invalid!");
		return TEXT("");
	}

	if (!SteamNetworkingUtils() || !SocketSub)
	{
		UE_LOGF(LogNet, Warning, "SteamSockets: Could not get host ping data as SteamNetworking is not available.");
		return TEXT("");
	}

	SteamNetworkPingLocation_t PingData;
	// This will use whatever latest ping data we have, even if we're recalculating it.
	// This should be near constant time.
	float PingDataAge = SteamNetworkingUtils()->GetLocalPingLocation(PingData);
	// While rare, this could happen if we request ping before all the interfaces are ready to go.
	if (PingDataAge == -1)
	{
		// This is okay though, because the next call to this should have actual ping information.
		UE_LOGF(LogNet, Log, "SteamSockets: No ping data is available at this time");
		return TEXT("");
	}

	// Copy some data around so that we can send this data over the network
	// (This must be done as data calculated on one machine is incompatible over the wire)
	ANSICHAR ConversionBuffer[k_cchMaxSteamNetworkingPingLocationString];
	FMemory::Memzero(ConversionBuffer); // Zero this for safety.
	SteamNetworkingUtils()->ConvertPingLocationToString(PingData, ConversionBuffer, k_cchMaxSteamNetworkingPingLocationString);
	
	return FString(ANSI_TO_TCHAR(ConversionBuffer));
}

int32 FSteamSocketsPing::GetPingFromHostData(const FString& HostPingStr) const
{
	if (!SteamNetworkingUtils() || HostPingStr.IsEmpty() || !IsUsingP2PRelays())
	{
		UE_LOGF(LogNet, Warning, "SteamSockets: Could not determine ping data as SteamNetworking is unavailable.");
		return -1;
	}

	// Attempt to serialize the data that we've gotten already.
	SteamNetworkPingLocation_t HostPingData;
	if (SteamNetworkingUtils()->ParsePingLocationString(TCHAR_TO_ANSI(*HostPingStr), HostPingData))
	{
		// This should be an extremely quick function, as it compares regional data within the node network
		// that we got from the host to our own node information. This ping data is mostly just math.
		int32 PingValue = SteamNetworkingUtils()->EstimatePingTimeFromLocalHost(HostPingData);
		if (PingValue == -1)
		{
			// A value of -1 occurs when there's a critical failure (We don't have our own ping data ready).
			// However, it's not a fatal error, and we can always request for that data again later.
			return -1;
		}

		return PingValue;
	}
	else
	{
		// This happens if the string has been tampered with, the protocol has upgraded or just something overall went wrong.
		UE_LOGF(LogNet, Log, "SteamSockets: Could not parse the host location in order to determine ping.");
		return -1;
	}
}



bool FSteamSocketsPing::IsRecalculatingPing() const
{
	if (!SteamNetworkingUtils() && IsUsingP2PRelays())
	{
		return false;
	}

	ESteamNetworkingAvailability NetworkStatus = SteamNetworkingUtils()->GetRelayNetworkStatus(nullptr);
	return NetworkStatus == k_ESteamNetworkingAvailability_Retrying || NetworkStatus == k_ESteamNetworkingAvailability_Waiting || NetworkStatus == k_ESteamNetworkingAvailability_Attempting;
}
