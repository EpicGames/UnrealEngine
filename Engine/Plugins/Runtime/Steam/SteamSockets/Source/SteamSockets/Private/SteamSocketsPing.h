// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlinePingInterfaceSteam.h"

/**
 * Steam Sockets ping interface implementation. See OnlinePingInterfaceSteam.h for information
 */
class FSteamSocketsPing : public FOnlinePingInterfaceSteam
{
public:
	FSteamSocketsPing(class FSteamSocketsSubsystem* InSocketSub, class FOnlineSubsystemSteam* InOnlineSub) :
		FOnlinePingInterfaceSteam(InOnlineSub),
		SocketSub(InSocketSub)
	{
	}
	
	virtual bool IsUsingP2PRelays() const override;
	virtual FString GetHostPingData() const override;
	virtual int32 GetPingFromHostData(const FString& HostPingStr) const override;
	virtual bool IsRecalculatingPing() const override;
	
protected:
	class FSteamSocketsSubsystem* SocketSub;
};
