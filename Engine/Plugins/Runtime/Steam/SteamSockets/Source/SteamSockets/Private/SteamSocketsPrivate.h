// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

// To make less changes to the Steam OSS, we'll define this here
// This will allow us to easily snap into the Steam OSS code
#ifndef STEAM_URL_PREFIX
#define STEAM_URL_PREFIX TEXT("steam.")
#endif

THIRD_PARTY_INCLUDES_START
// IWYU pragma: begin_keep

// Main headers
#include "steam/steam_api.h"
#include "steam/steam_gameserver.h"

// Socket specific headers
#include "steam/isteamnetworkingsockets.h"
#include "steam/isteamnetworkingutils.h"
#include "steam/steamnetworkingtypes.h"

// IWYU pragma: end_keep
THIRD_PARTY_INCLUDES_END

/** Some flags in the Steam SDK share similar numeral values which could change in the future.
 *  Static analysis does not understand the difference and instead sees code like Object == 1 || Object == 1
 *  For safety, we need to check both constants as the values in the SDK could change in a future version.
 *  So this macro block was introduced to silence value redundancy warnings in static analysis.
 */
#ifdef _MSC_VER
#define STEAM_SDK_IGNORE_REDUNDANCY_START \
		__pragma(warning(push)) \
		__pragma(warning(disable: 6287)) 

#define STEAM_SDK_IGNORE_REDUNDANCY_END \
		__pragma(warning(pop))
#else
#define STEAM_SDK_IGNORE_REDUNDANCY_START
#define STEAM_SDK_IGNORE_REDUNDANCY_END
#endif
