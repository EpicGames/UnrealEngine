// Copyright Epic Games, Inc. All Rights Reserved.

#include "SteamSocketsSubsystem.h"
#include "SteamSocketsPackage.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "SteamSocketsTaskManager.h"
#include "SteamSocketsNetDriver.h"
#include "SteamSharedModule.h"
#include "SteamSocket.h"
#include "SteamSocketsPing.h"
#include "OnlineSubsystemSteam.h"
#include "SteamSocketsTypes.h" // IWYU pragma: keep

// Log Category for The API Debugger
DEFINE_LOG_CATEGORY_STATIC(LogSteamSocketsAPI, Log, All);

/** Message logging hook for debugging Steam Sockets. */
void SteamSocketsDebugLogger(ESteamNetworkingSocketsDebugOutputType nType, const char *pszMsg)
{
#if !NO_LOGGING
	FString OutputType;
	ELogVerbosity::Type Verboseness = ELogVerbosity::Verbose;
	switch (nType)
	{
		default:
		case k_ESteamNetworkingSocketsDebugOutputType_None:
			OutputType = TEXT("None");
			break;
		case k_ESteamNetworkingSocketsDebugOutputType_Bug:
			OutputType = TEXT("Bug");
			Verboseness = ELogVerbosity::Error;
			break;
		case k_ESteamNetworkingSocketsDebugOutputType_Error:
			OutputType = TEXT("Error");
			Verboseness = ELogVerbosity::Error;
			break;
		case k_ESteamNetworkingSocketsDebugOutputType_Important:
			OutputType = TEXT("Important");
			Verboseness = ELogVerbosity::Warning;
			break;
		case k_ESteamNetworkingSocketsDebugOutputType_Warning:
			OutputType = TEXT("Warning");
			Verboseness = ELogVerbosity::Warning;
			break;
		case k_ESteamNetworkingSocketsDebugOutputType_Everything:
		case k_ESteamNetworkingSocketsDebugOutputType_Verbose:
		case k_ESteamNetworkingSocketsDebugOutputType_Msg:
			OutputType = TEXT("Log");
			break;
		case k_ESteamNetworkingSocketsDebugOutputType_Debug:
			OutputType = TEXT("Debug");
			break;
	}

	// Runtime detection of logging level and verbosity.
	GLog->Log(LogSteamSocketsAPI.GetCategoryName(), Verboseness, FString::Printf(TEXT("SteamSockets API: %s %s"), *OutputType, ANSI_TO_TCHAR(pszMsg)));
#endif
}

FSteamSocketsSubsystem* FSteamSocketsSubsystem::SocketSingleton = nullptr;

FSteamSocketsSubsystem* FSteamSocketsSubsystem::Create()
{
	if (SocketSingleton == nullptr)
	{
		SocketSingleton = new FSteamSocketsSubsystem();
	}

	return SocketSingleton;
}

void FSteamSocketsSubsystem::Destroy()
{
	if (SocketSingleton != nullptr)
	{
		SocketSingleton->Shutdown();
		delete SocketSingleton;
		SocketSingleton = nullptr;
	}
}

bool FSteamSocketsSubsystem::Init(FString& Error)
{
	const bool bIsDedicated = IsRunningDedicatedServer();
	if (GConfig)
	{
		if (!GConfig->GetBool(TEXT("OnlineSubsystemSteam"), TEXT("bAllowP2PPacketRelay"), bUseRelays, GEngineIni))
		{
			UE_CLOGF(bIsDedicated, LogSockets, Warning, "SteamSockets: Missing config value for bAllowP2PPacketRelay, will be unable to determine ping functionality");
		}
	}

	// Debug commandline support for socket relays. 
#if !UE_BUILD_SHIPPING
	bool bOverrideRelays = false;
	if (FParse::Bool(FCommandLine::Get(), TEXT("SteamSocketsRelays"), bOverrideRelays))
	{
		UE_LOGF(LogSockets, Log, "SteamSockets: Set relay setting to %d", bOverrideRelays);
		bUseRelays = bOverrideRelays;
	}
#endif

	// Get the online subsystem
	FOnlineSubsystemSteam* OnlineSteamSubsystem = static_cast<FOnlineSubsystemSteam*>(IOnlineSubsystem::Get(STEAM_SUBSYSTEM));

	// Initialize the Server API if we are going to be using it.
	if (bIsDedicated)
	{
		SteamAPIServerHandle = FSteamSharedModule::Get().ObtainSteamServerInstanceHandle();
		if (!SteamAPIServerHandle.IsValid())
		{
			Error = TEXT("SteamSockets: Could not initialize the game server SteamAPI!");
			SteamAPIServerHandle.Reset();
			return false;
		}

		// Register the GameServer login status call for when the session system logs us in.
		if (OnlineSteamSubsystem != nullptr)
		{
			SteamServerLoginDelegateHandle = OnlineSteamSubsystem->AddOnSteamServerLoginCompletedDelegate_Handle(
				FOnSteamServerLoginCompletedDelegate::CreateRaw(this, &FSteamSocketsSubsystem::OnServerLoginComplete));
		}
		else
		{
			Error = TEXT("SteamSockets: Cannot register dedicated server login listeners!");
			SteamAPIServerHandle.Reset();
			return false;
		}
	}
	else
	{
		SteamAPIClientHandle = FSteamSharedModule::Get().ObtainSteamClientInstanceHandle();
		if (!SteamAPIClientHandle.IsValid())
		{
			Error = TEXT("SteamSockets: could not obtain a handle to SteamAPI!");
			SteamAPIClientHandle.Reset();
			return false;
		}
	}

	// These functions cause access violations on application exit due to how they spin up.
	// When fixed in a new version of the Steam client application, this block can be removed.
#if !PLATFORM_LINUX
	// Clients and servers using the relay network will want to set this up
	if (IsUsingRelayNetwork())
	{
		// We need this functionality in order to modify socket configuration and determine ping data.
		if (!SteamNetworkingUtils())
		{
			Error = TEXT("SteamSockets: Cannot interface with SteamNetworkingUtils! Relay support relies on this functionality!");
			SteamAPIClientHandle.Reset();
			SteamAPIServerHandle.Reset();
			return false;
		}

		// We do not care about the return of this function the first time it runs.
		UE_LOGF(LogSockets, Log, "SteamSockets: Initializing Network Relay");
		SteamNetworkingUtils()->InitRelayNetworkAccess();
	}

	// Start up the authentication system so we get the signing certs for this application session.
	if (GetSteamSocketsInterface())
	{
		GetSteamSocketsInterface()->InitAuthentication();
	}
#endif
	
	// Set up the API Logging
	ESteamNetworkingSocketsDebugOutputType DebugLevel = k_ESteamNetworkingSocketsDebugOutputType_Msg;

	// Set this a bit higher for shipping builds because SteamSockets can spew a lot of information
#if UE_BUILD_SHIPPING
	DebugLevel = k_ESteamNetworkingSocketsDebugOutputType_Important;
#endif

	SteamNetworkingUtils()->SetDebugOutputFunction(DebugLevel, SteamSocketsDebugLogger);

	// Hook up the Steam Event Manager so we can get connection updates.
	SteamEventManager = MakeUnique<FSteamSocketsTaskManager>(this);

	// Set the ping interface up, this will allow the Steam OSS to compile without direct links to this module.
	if (OnlineSteamSubsystem != nullptr && IsUsingRelayNetwork())
	{
		// Deletion will be handled by the Steam OSS, we do not need to worry about it in this subsystem.
		OnlineSteamSubsystem->SetPingInterface(MakeShared<FSteamSocketsPing, ESPMode::ThreadSafe>(this, OnlineSteamSubsystem));
	}

	return true;
}

void FSteamSocketsSubsystem::Shutdown()
{
	UE_LOGF(LogSockets, Log, "SteamSockets: Cleaning up");

	// Clear our delegate handles
	FOnlineSubsystemSteam* OnlineSteamSubsystem = static_cast<FOnlineSubsystemSteam*>(IOnlineSubsystem::Get(STEAM_SUBSYSTEM));
	if (OnlineSteamSubsystem)
	{
		OnlineSteamSubsystem->ClearOnSteamServerLoginCompletedDelegate_Handle(SteamServerLoginDelegateHandle);
		OnlineSteamSubsystem->SetPingInterface(nullptr);
	}

	// Clean up the internal event manager.
	if (SteamEventManager.IsValid())
	{
		SteamEventManager.Reset();
	}

	// Remove all our listeners that were pending.
	PendingListenerArray.Empty();

	// Let go of our API handles. This is quicker than waiting for destruction
	SteamAPIClientHandle.Reset();
	SteamAPIServerHandle.Reset();
}

class FSocket* FSteamSocketsSubsystem::CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolType)
{
	FName ProtocolTypeToUse = ProtocolType;
	if (ProtocolTypeToUse.IsNone())
	{
		if (IsUsingRelayNetwork())
		{
			ProtocolTypeToUse = FNetworkProtocolTypes::SteamSocketsP2P;
		}
		else
		{
			// In addition to the P2P framework, the Steam Sockets can also work over IP.
			// Generally, it's better to use the UIpNetDriver, but to prevent us from crashing
			// if you have relays disabled, the fallback is the IP framework.
			ProtocolTypeToUse = FNetworkProtocolTypes::SteamSocketsIP;
		}
	}
	else if (ProtocolType != FNetworkProtocolTypes::SteamSocketsP2P && ProtocolType != FNetworkProtocolTypes::SteamSocketsIP)
	{
		UE_LOGF(LogSockets, Warning, "SteamSockets: Attempted to create a socket with a protocol outside of the steam network!");
		return nullptr;
	}

	return static_cast<FSocket*>(new FSteamSocket(SOCKTYPE_Streaming, SocketDescription, ProtocolTypeToUse));
}

void FSteamSocketsSubsystem::DestroySocket(FSocket* Socket)
{
	if (Socket != nullptr)
	{
		FSteamSocket* SteamSocket = static_cast<FSteamSocket*>(Socket);
		UE_LOGF(LogSockets, Verbose, "SteamSockets: Destroy socket called on %u", SteamSocket->InternalHandle);

		// Save the internal handle because it will be invalid soon.
		FSteamSocketInformation* SocketInfo = GetSocketInfo(SteamSocket->InternalHandle);
		if (SocketInfo != nullptr)
		{
			// Warn the NetDriver that we're destructing the socket such that nothing else tampers with it.
			USteamSocketsNetDriver* SteamNetDriver = SocketInfo->NetDriver.Get();
			if (SteamNetDriver)
			{
				SteamNetDriver->ResetSocketInfo(SteamSocket);
			}
			SocketInfo->Socket = nullptr;
			SocketInfo->MarkForDeletion(); // Mark us for deletion so we're removed from the map
		}

		// Socket closure will properly mark the socket for pending removal.
		Socket->Close();

		delete Socket;
	}
}

FAddressInfoResult FSteamSocketsSubsystem::GetAddressInfo(const TCHAR* HostName, const TCHAR* ServiceName, 
	EAddressInfoFlags QueryFlags, const FName ProtocolTypeName, ESocketType SocketType)
{
	// We don't support queries but if we can make an address from it then that's good enough.
	FAddressInfoResult ResultData(HostName, ServiceName);

	// Determine if we're asking for any specific protocol.
	const bool bWildcardProtocol = ProtocolTypeName.IsNone();

	// If you requested anything outside of the steam network, we will be unable to support it.
	// This occurs when you have:
	// * Given no hostname and service name information
	// * Given a service name but the service name is not numeric (SteamNetwork can only take port numbers)
	// * Specified a protocol type that's not a part of the steam network.
	if ((HostName == nullptr && ServiceName == nullptr) || (ServiceName != nullptr && !FCString::IsNumeric(ServiceName)) ||
		(ProtocolTypeName != FNetworkProtocolTypes::SteamSocketsP2P && ProtocolTypeName != FNetworkProtocolTypes::SteamSocketsIP && !bWildcardProtocol))
	{
		ResultData.ReturnCode = SE_EINVAL;
		return ResultData;
	}

	TArray<TSharedPtr<FInternetAddr>> AddressList;
	// Handle binding flags or if you don't pass in a hostname
	if (HostName == nullptr || EnumHasAnyFlags(QueryFlags, EAddressInfoFlags::BindableAddress))
	{
		// Grab all the adapters, we'll process them later.
		// We don't care about the return type as on SteamSockets this call cannot fail.
		GetLocalAdapterAddresses(AddressList);
	}
	else
	{
		TSharedPtr<FInternetAddr> SerializedAddr = GetAddressFromString(HostName);
		// Only add to this list if the serialization succeeded.
		if (SerializedAddr.IsValid())
		{
			AddressList.Add(SerializedAddr);
		}
	}
	
	if (AddressList.Num() >= 1)
	{
		// Assume we have no error, we'll check for this later.
		ResultData.ReturnCode = SE_NO_ERROR;

		// Preprocess the port information so we can set it later.
		int32 PortToUse = -1;
		if (ServiceName != nullptr && FCString::IsNumeric(ServiceName))
		{
			PortToUse = FCString::Atoi(ServiceName);
		}

		// Process the address list.
		for (auto& AddressItem : AddressList)
		{
			// Check if the types match (add them anyways if protocol type wasn't specified).
			if (bWildcardProtocol || (!bWildcardProtocol && AddressItem->GetProtocolType() == ProtocolTypeName))
			{
				// We will be writing directly into the array but this doesn't matter in our case as all the addresses are shared allocations.
				if (PortToUse >= 0)
				{
					AddressItem->SetPort(PortToUse);
				}
				ResultData.Results.Add(FAddressInfoResultData(AddressItem.ToSharedRef(), 0, AddressItem->GetProtocolType(), SOCKTYPE_Streaming));
			}
		}

		// If our address list is essentially empty due to mismatches, then set the correct error flag.
		if (ResultData.Results.Num() <= 0)
		{
			if (!bWildcardProtocol)
			{
				// We couldn't find addresses that gave what the user was looking for.
				ResultData.ReturnCode = SE_ADDRFAMILY;
			}
			else
			{
				// Otherwise something unforeseen happened.
				ResultData.ReturnCode = SE_EFAULT;
			}
		}
	}
	else
	{
		ResultData.ReturnCode = SE_NO_DATA;
	}

	return ResultData;
}

TSharedPtr<FInternetAddr> FSteamSocketsSubsystem::GetAddressFromString(const FString& IPAddress)
{
	TSharedRef<FInternetAddrSteamSockets> NewAddr = StaticCastSharedRef<FInternetAddrSteamSockets>(CreateInternetAddr());

	// Passing an empty string should just map it to the any address.
	if (IPAddress.IsEmpty())
	{
		// NOTE: There's a lot of questions as to what this address should be here.
		// Should we be returning the identity or the any address, 
		// there's not really a good way to determine default.
		NewAddr->SetAnyAddress();
		
		return NewAddr;
	}

	bool bIsAddrValid = false;
	NewAddr->SetIp(*IPAddress, bIsAddrValid);
	if (!bIsAddrValid)
	{
		return nullptr;
	}

	return NewAddr;
}

bool FSteamSocketsSubsystem::GetHostName(FString& HostName)
{
	// We have no way of looking up addresses on this platform.
	UE_LOGF(LogSockets, Warning, "GetHostName is not supported on SteamSockets");
	return false;
}

TSharedRef<FInternetAddr> FSteamSocketsSubsystem::CreateInternetAddr()
{
	return MakeShareable(new FInternetAddrSteamSockets());
}

TSharedRef<FInternetAddr> FSteamSocketsSubsystem::CreateInternetAddr(const FName RequestedProtocol)
{
	return MakeShareable(new FInternetAddrSteamSockets(RequestedProtocol));
}

const TCHAR* FSteamSocketsSubsystem::GetSocketAPIName() const
{
	return TEXT("SteamSockets");
}

bool FSteamSocketsSubsystem::GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr>>& OutAddresses)
{
	// Add the account addresses
	if (IsUsingRelayNetwork())
	{
		TSharedPtr<FInternetAddr> IdentityAddress = GetIdentityAddress();
		if (IdentityAddress.IsValid())
		{
			OutAddresses.Add(IdentityAddress);
		}
	}

	// Always include the any address.
	TSharedRef<FInternetAddr> AnyAddress = CreateInternetAddr(FNetworkProtocolTypes::SteamSocketsIP);
	AnyAddress->SetAnyAddress();
	OutAddresses.Add(AnyAddress);

	return true;
}

TArray<TSharedRef<FInternetAddr>> FSteamSocketsSubsystem::GetLocalBindAddresses()
{
	TArray<TSharedRef<FInternetAddr>> OutAddresses;
	TArray<TSharedPtr<FInternetAddr>> AdapterAddresses;
	GetLocalAdapterAddresses(AdapterAddresses);

	// Add Multihome
	TSharedRef<FInternetAddr> MultihomeAddress = CreateInternetAddr(FNetworkProtocolTypes::SteamSocketsIP);
	if (GetMultihomeAddress(MultihomeAddress))
	{
		OutAddresses.Add(MultihomeAddress);
	}

	// Add all the adapter addresses
	for (const auto& AdapterAddress : AdapterAddresses)
	{
		OutAddresses.Add(AdapterAddress.ToSharedRef());
	}

	return OutAddresses;
}

void FSteamSocketsSubsystem::CleanSocketInformation(bool bForceClean)
{
	for (SocketHandleInfoMap::TIterator It(SocketInformationMap); It; ++It)
	{
		FSteamSocketInformation& SocketInfo = It.Value();
		if (SocketInfo.IsMarkedForDeletion() || bForceClean)
		{
			// Close the netdriver and any connections.
			USteamSocketsNetDriver* NetDriverInfo = SocketInfo.NetDriver.Get();
			if (NetDriverInfo && SocketInfo.Parent == nullptr)
			{
				NetDriverInfo->Shutdown();
				SocketInfo.NetDriver.Reset();
			}

			// Close the socket (frees internal memory)
			//
			// Deletion with SteamSockets is fairly safe. Multiple paths can mark a socket 
			// for deletion but it will only ever be deleted once.
			if (SocketInfo.Socket)
			{
				DestroySocket(SocketInfo.Socket);
			}

			It.RemoveCurrent();
		}
	}
}

void FSteamSocketsSubsystem::DumpSocketInformationMap() const
{
#if !UE_BUILD_SHIPPING
	if (SocketInformationMap.Num() < 1)
	{
		UE_LOGF(LogSockets, Log, "SteamSockets: Socket Information Map is empty!");
		return;
	}

	UE_LOGF(LogSockets, Log, "SteamSockets: Printing Socket Information Map:\n");
	for (SocketHandleInfoMap::TConstIterator It(SocketInformationMap); It; ++It)
	{
		const FSteamSocketInformation& SocketInfo = It.Value();
		UE_LOGF(LogSockets, Log, "# %ls", *SocketInfo.ToString());
	}
#endif
}

bool FSteamSocketsSubsystem::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FSteamSocketsSubsystem_Tick);

	// Handle all of our updates from the Steam API callbacks
	if (SteamEventManager.IsValid())
	{
		SteamEventManager->Tick();
	}

	CleanSocketInformation(false);
	return true;
}

bool FSteamSocketsSubsystem::Exec_Dev(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bIsHandled = false;

#if !UE_BUILD_SHIPPING
	if (FParse::Command(&Cmd, TEXT("PrintSteamSocketInfo")))
	{
		bIsHandled = true;
		DumpSocketInformationMap();
	}
	if (FParse::Command(&Cmd, TEXT("PrintPendingSteamSocketInfo")))
	{
		bIsHandled = true;
		UE_LOGF(LogSockets, Log, "SteamSockets: Printing Pending Socket Info");
		for (auto& Pending : PendingListenerArray)
		{
			UE_LOGF(LogSockets, Log, "# %ls", *Pending.ToString());
		}
	}
	else if (FParse::Command(&Cmd, TEXT("ClearSteamSocketInfo")))
	{
		UE_LOGF(LogSockets, Log, "SteamSockets: Clearing all socket information!");
		bIsHandled = true;
		CleanSocketInformation(true);
	}
	else if (FParse::Command(&Cmd, TEXT("TogglePeekMessaging")))
	{
		bIsHandled = true;
		bShouldTestPeek = !bShouldTestPeek;
		UE_LOGF(LogSockets, Log, "SteamSockets: Set Peek Messaging to %d", bShouldTestPeek);
	}
#endif

	return bIsHandled;
}

ISteamNetworkingSockets* FSteamSocketsSubsystem::GetSteamSocketsInterface()
{
	return (SteamGameServerNetworkingSockets() != nullptr && IsRunningDedicatedServer()) ? SteamGameServerNetworkingSockets() : SteamNetworkingSockets();
}

bool FSteamSocketsSubsystem::FSteamSocketInformation::operator==(const FInternetAddr& InAddr) const
{
	FInternetAddrSteamSockets SteamAddr = *((FInternetAddrSteamSockets*)&InAddr);
	return *Addr == SteamAddr;
}

void FSteamSocketsSubsystem::FSteamSocketInformation::MarkForDeletion()
{
	bMarkedForDeletion = true;
	USteamSocketsNetDriver* NetDriverObj = NetDriver.Get();
	if (Socket != nullptr && NetDriverObj)
	{
		UE_LOGF(LogSockets, Verbose, "SteamSockets: Handle %u marked for deletion. Has parent? %d NetDriver definition: %ls", 
			Socket->InternalHandle, (Parent != nullptr), *NetDriverObj->GetDescription());
	}
}

FString FSteamSocketsSubsystem::FSteamSocketInformation::ToString() const
{
	return FString::Printf(TEXT("SocketInfo: Addr[%s], Socket[%u], Status[%d], Listener[%d], HasNetDriver[%d], MarkedForDeletion[%d]"), 
		(Addr.IsValid() ? *Addr->ToString(true) : TEXT("INVALID")), (Socket != nullptr ? Socket->InternalHandle : 0), 
		(Socket != nullptr ? (int32)Socket->GetConnectionState() : -1),
		(Parent != nullptr), 
		NetDriver.IsValid(), bMarkedForDeletion);
}

FString FSteamSocketsSubsystem::FSteamPendingSocketInformation::ToString() const
{
	USteamSocketsNetDriver* SteamNetDriver = NetDriver.Get();
	return FString::Printf(TEXT("PendingSocketInfo: NetDriver name %s"), (SteamNetDriver != nullptr) ? *SteamNetDriver->GetDescription() : TEXT("INVALID"));
}

FSteamSocketsSubsystem::FSteamSocketInformation* FSteamSocketsSubsystem::GetSocketInfo(SteamSocketHandles InternalSocketHandle)
{
	if (InternalSocketHandle == k_HSteamNetConnection_Invalid)
	{
		UE_LOGF(LogSockets, Warning, "SteamSockets: Cannot get information on an invalid socket handle, returning null");
		return nullptr;
	}

	return SocketInformationMap.Find(InternalSocketHandle);
}

FSteamSocketsSubsystem::FSteamSocketInformation* FSteamSocketsSubsystem::GetSocketInfo(const FInternetAddr& ForAddress)
{
	// Small time save with invalid data
	if (!ForAddress.IsValid())
	{
		return nullptr;
	}

	// This functionality is a bit slower, because we have to do complete address lookups through the map.
	// It's much better to use the handles to find the socket info. There's also a possibility for collisions
	// here as well. This is why internally, this function is not used as much (only when we don't have enough data).
	for (SocketHandleInfoMap::TIterator It(SocketInformationMap); It; ++It)
	{
		if (It.Value() == ForAddress)
		{
			return &(It.Value());
		}
	}

	return nullptr;
}

void FSteamSocketsSubsystem::AddSocket(const FInternetAddr& ForAddr, FSteamSocket* NewSocket, FSteamSocket* ParentSocket)
{
	if (NewSocket == nullptr || ForAddr.IsValid() == false)
	{
		UE_LOGF(LogSockets, Warning, "SteamSockets: Attempted to track invalid socket data!");
		return;
	}

	STEAM_SDK_IGNORE_REDUNDANCY_START
	// Do not attempt to add socket data for sockets that have invalid handles.
	if (NewSocket->InternalHandle == k_HSteamListenSocket_Invalid || NewSocket->InternalHandle == k_HSteamNetConnection_Invalid)
	{
		UE_LOGF(LogSockets, Verbose, "SteamSockets: Dropped socket tracking for socket with invalid handle.");
		return;
	}
	STEAM_SDK_IGNORE_REDUNDANCY_END

	FSteamSocketInformation NewSocketInfo(ForAddr.Clone(), NewSocket, ParentSocket);
	if (SocketInformationMap.Find(NewSocket->InternalHandle) == nullptr)
	{
		UE_LOGF(LogSockets, Log, "SteamSockets: Now tracking socket %u for addr %ls, has parent? %d", NewSocket->InternalHandle, *ForAddr.ToString(true), (ParentSocket != nullptr));
		SocketInformationMap.Add(NewSocket->InternalHandle, NewSocketInfo);
	}
	else
	{
		UE_LOGF(LogSockets, Warning, "SteamSockets: A socket with the address %ls already exists, will not add another!", *ForAddr.ToString(true));
	}
}

void FSteamSocketsSubsystem::RemoveSocketsForListener(FSteamSocket* ListenerSocket)
{
	if (ListenerSocket == nullptr)
	{
		UE_LOGF(LogSockets, Warning, "SteamSockets: Attempted to free sockets attached to an invalid listener!");
		return;
	}

	// When a listen socket closes in the SteamAPI, all sockets that were created from that listener are ungracefully closed.
	// However this does not free any memory, nor does it alert any other object. As such, we have to process this destruction here.
	UE_LOGF(LogSockets, Log, "SteamSockets: Closing all sockets attached to listener %u", ListenerSocket->InternalHandle);
	for (SocketHandleInfoMap::TIterator It(SocketInformationMap); It; ++It)
	{
		FSteamSocketInformation& SocketInfo = It.Value();
		if (SocketInfo.Parent == ListenerSocket)
		{
			UE_LOGF(LogSockets, Verbose, "SteamSockets: Removed socket %u", SocketInfo.Socket->InternalHandle);
			// A loop later will clean out these connections, shutting them down properly.
			// This is to prevent acting on deletion events from the API.
			SocketInfo.Parent = nullptr;
			// The children also should not be cleaning up any netdrivers they don't own
			SocketInfo.NetDriver.Reset();
			SocketInfo.MarkForDeletion();
		}
	}
}

void FSteamSocketsSubsystem::QueueRemoval(SteamSocketHandles RemoveHandle)
{
	FSteamSocketInformation* SocketInfo = GetSocketInfo(RemoveHandle);
	if (SocketInfo != nullptr)
	{
		FString Address = (SocketInfo->Addr.IsValid()) ? SocketInfo->Addr->ToString(true) : TEXT("INVALID");
		UE_LOGF(LogSockets, Verbose, "SteamSockets: Marked socket %u with address %ls for removal (pending)", RemoveHandle, *Address);
		SocketInfo->MarkForDeletion();
	}
}

void FSteamSocketsSubsystem::LinkNetDriver(FSocket* Socket, USteamSocketsNetDriver* NewNetDriver)
{
	if (Socket == nullptr || NewNetDriver == nullptr)
	{
		UE_LOGF(LogSockets, Warning, "SteamSockets: Attempted to create an invalid socket/netdriver pairing!");
		return;
	}

	FSteamSocket* SteamSocket = static_cast<FSteamSocket*>(Socket);
	// Do not attempt to track netdrivers with invalid sockets.
	STEAM_SDK_IGNORE_REDUNDANCY_START
	if (SteamSocket->InternalHandle == k_HSteamListenSocket_Invalid || SteamSocket->InternalHandle == k_HSteamNetConnection_Invalid)
	{
		UE_LOGF(LogSockets, Verbose, "SteamSockets: Dropped netdriver link with socket that has invalid handle.");
		return;
	}
	STEAM_SDK_IGNORE_REDUNDANCY_END

	// Link the netdriver to this socket.
	FSteamSocketInformation* SocketInfo = GetSocketInfo(SteamSocket->InternalHandle);
	if (SocketInfo != nullptr)
	{
		SocketInfo->NetDriver = NewNetDriver;
	}
}

void FSteamSocketsSubsystem::AddDelayedListener(FSteamSocket* ListenSocket, USteamSocketsNetDriver* NewNetDriver)
{
	// Ignore any invalid data.
	if (ListenSocket == nullptr || NewNetDriver == nullptr)
	{
		UE_LOGF(LogSockets, Warning, "SteamSockets: Add delayed listener cannot listen to invalid data!");
		return;
	}

	FSteamPendingSocketInformation WatchedListener;
	WatchedListener.Socket = ListenSocket;

	// Track the netdriver
	WatchedListener.NetDriver = NewNetDriver;

	// Push to the array list
	PendingListenerArray.Add(WatchedListener);
}

void FSteamSocketsSubsystem::OnServerLoginComplete(bool bWasSuccessful)
{
	TSharedPtr<FInternetAddrSteamSockets> SteamIdentityAddr = StaticCastSharedPtr<FInternetAddrSteamSockets>(GetIdentityAddress());
	for (auto& DelayedListener : PendingListenerArray)
	{
		USteamSocketsNetDriver* SteamNetDriver = DelayedListener.NetDriver.Get();
		if (SteamNetDriver == nullptr)
		{
			continue;
		}

		// If we are not successful, we need to get out.
		if (!bWasSuccessful)
		{
			// Mark the NetDriver for cleanup.
			SteamNetDriver->Shutdown();
			continue;
		}

		// Update the binding address
		if (SteamIdentityAddr.IsValid())
		{
			// Push in the new valid information.
			int32 ListenPort = DelayedListener.Socket->BindAddress.GetPlatformPort();
			TSharedPtr<FInternetAddrSteamSockets> ListenerAddress = StaticCastSharedRef<FInternetAddrSteamSockets>(SteamIdentityAddr->Clone());
			ListenerAddress->SetPlatformPort(ListenPort);

			// Update the socket and netdriver addresses
			DelayedListener.Socket->BindAddress = *ListenerAddress;
			SteamNetDriver->LocalAddr = ListenerAddress;

			// Actually start the listen call
			if (!DelayedListener.Socket->Listen(0))
			{
				// Connection failed, fire shutdown.
				SteamNetDriver->Shutdown();
				continue;
			}
		}

		// Link the netdriver information so that we're ready.
		LinkNetDriver(DelayedListener.Socket, SteamNetDriver);

		// Toggle off the netdriver's tick throttle.
		SteamNetDriver->bIsDelayedNetworkAccess = false;

		UE_LOGF(LogSockets, Verbose, "SteamSockets: Processed delayed listener: %ls", *DelayedListener.ToString());
	}

	PendingListenerArray.Empty();
}

TSharedPtr<FInternetAddr> FSteamSocketsSubsystem::GetIdentityAddress()
{
	SteamNetworkingIdentity SteamIDData;
	TSharedRef<FInternetAddrSteamSockets> SteamAddr = StaticCastSharedRef<FInternetAddrSteamSockets>(CreateInternetAddr(FNetworkProtocolTypes::SteamSocketsP2P));

	// Attempt to get the machine's identity (their steam ID)
	if (GetSteamSocketsInterface() && GetSteamSocketsInterface()->GetIdentity(&SteamIDData))
	{
		// If we got it, assign it to the SteamAddr
		SteamAddr->Addr = SteamIDData;
		return SteamAddr;
	}
	else if (!IsRunningDedicatedServer() && SteamUser())
	{
		CSteamID CurrentUser = SteamUser()->GetSteamID();
		if (CurrentUser.IsValid())
		{
			SteamAddr->Addr.SetSteamID(CurrentUser);
		}
		else
		{
			UE_LOGF(LogSockets, Warning, "SteamSockets: Unable to process current user's steam id!");
		}
	}

	return nullptr;
}

bool FSteamSocketsSubsystem::IsLoggedInToSteam() const
{
	// Check the game server functionality first because servers should be prioritized over SteamUser calls if
	// the server has linked with the client dlls via launch arguments
	return ((SteamGameServer() && IsRunningDedicatedServer()) ? SteamGameServer()->BLoggedOn() : ((SteamUser()) ? SteamUser()->BLoggedOn() : false));
}

// Helper function to translate Valve connection state messages into something human readable.
static FString ConnectionStateToString(const ESteamNetworkingConnectionState& ConnectionState)
{
	switch (ConnectionState)
	{
	default:
	case k_ESteamNetworkingConnectionState_None:
		return TEXT("None");
	case k_ESteamNetworkingConnectionState_Connecting:
		return TEXT("Connecting");
	case k_ESteamNetworkingConnectionState_FindingRoute:
		return TEXT("Finding Route");
	case k_ESteamNetworkingConnectionState_Connected:
		return TEXT("Connected");
	case k_ESteamNetworkingConnectionState_ClosedByPeer:
		return TEXT("Closed by Peer");
	case k_ESteamNetworkingConnectionState_ProblemDetectedLocally:
		return TEXT("Local Connection Issue");
	}
}

void FSteamSocketsSubsystem::SteamSocketEventHandler(struct SteamNetConnectionStatusChangedCallback_t* Message)
{
	if (Message == nullptr || FSteamSocketsSubsystem::GetSteamSocketsInterface() == nullptr)
	{
		return;
	}

	// New Connection came in
	if (Message->m_eOldState == k_ESteamNetworkingConnectionState_None &&
		Message->m_info.m_eState == k_ESteamNetworkingConnectionState_Connecting &&
		Message->m_info.m_hListenSocket != k_HSteamListenSocket_Invalid)
	{
		// Push this to the netdriver.
		if (FSteamSocketInformation* SocketInfo = GetSocketInfo(Message->m_info.m_hListenSocket))
		{
			if (USteamSocketsNetDriver* NetDriver = SocketInfo->NetDriver.Get())
			{
				NetDriver->OnConnectionCreated(Message->m_info.m_hListenSocket, Message->m_hConn);
			}
		}
	}
	else if ((Message->m_eOldState == k_ESteamNetworkingConnectionState_Connecting || Message->m_eOldState == k_ESteamNetworkingConnectionState_FindingRoute) &&
		Message->m_info.m_eState == k_ESteamNetworkingConnectionState_Connected)
	{
		// Connection has been established
		if (FSteamSocketInformation* SocketInfo = GetSocketInfo(Message->m_hConn))
		{
			if (USteamSocketsNetDriver* NetDriver = SocketInfo->NetDriver.Get())
			{
				NetDriver->OnConnectionUpdated(Message->m_hConn, (int32)Message->m_info.m_eState);
			}
		}
	}
	else if ((Message->m_eOldState == k_ESteamNetworkingConnectionState_FindingRoute || Message->m_eOldState == k_ESteamNetworkingConnectionState_Connecting 
		|| Message->m_eOldState == k_ESteamNetworkingConnectionState_Connected) &&
		(Message->m_info.m_eState == k_ESteamNetworkingConnectionState_ClosedByPeer || Message->m_info.m_eState == k_ESteamNetworkingConnectionState_ProblemDetectedLocally))
	{
		FSteamSocketInformation* SocketInfo = GetSocketInfo(Message->m_hConn);
		// If we know anything about the socket, we need to act on it.
		if (SocketInfo != nullptr && SocketInfo->Socket != nullptr && !SocketInfo->Socket->bIsListenSocket)
		{
			// Print out information about the disconnection.
			UE_LOGF(LogSockets, Verbose, "SteamSockets: Connection %u has disconnected. Old state: %ls Reason: %ls", Message->m_hConn,
				*ConnectionStateToString(Message->m_eOldState), *ConnectionStateToString(Message->m_info.m_eState));

			// Let the netdriver know of the disconnection
			USteamSocketsNetDriver* NetDriver = SocketInfo->NetDriver.Get();
			if (NetDriver)
			{
				NetDriver->OnConnectionDisconnected(Message->m_hConn);
			}

			// If we close via the listen socket deletion then let the connection just die.
			// Do not alert anyone as we already know we're disconnected.
			//
			// To determine this case we follow these two rules:
			// * Servers will always mark all related sockets as not having a parent.
			// * Clients will never have valid listener socket handles in their message data.
			if (SocketInfo->Parent != nullptr || Message->m_info.m_hListenSocket == k_HSteamListenSocket_Invalid)
			{
				// If we are in here, it means we are a genuine disconnection, not caused by the listen socket destruction
				// Or any other event that we ourselves caused.
				//
				// Mark this to be cleaned up. The tick will handle actual deletion.
				SocketInfo->MarkForDeletion();
			}
		}
	}
}
