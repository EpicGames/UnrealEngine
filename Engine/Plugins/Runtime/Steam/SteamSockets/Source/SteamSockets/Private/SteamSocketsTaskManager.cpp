// Copyright Epic Games, Inc. All Rights Reserved.

#include "SteamSocketsTaskManager.h"
#include "SteamSocketsSubsystem.h"

void FSteamSocketsTaskManager::OnConnectionStatusChanged(SteamNetConnectionStatusChangedCallback_t* SteamConnectionMessage)
{
	// When we are called in here, we are likely going to be on the Steam OSS's online thread.
	// The Steam callback API will fire on whatever thread the OSS is on.
	//
	// Because of how the callback api works, we need to store copies of these messages, as the pointers will be invalid
	// the instant we leave our function.
	MessageQueue.Enqueue(*SteamConnectionMessage);
}

void FSteamSocketsTaskManager::OnConnectionStatusChangedGS(SteamNetConnectionStatusChangedCallback_t* SteamConnectionMessage)
{
	MessageQueue.Enqueue(*SteamConnectionMessage);
}

void FSteamSocketsTaskManager::Tick()
{
	SteamNetConnectionStatusChangedCallback_t CallbackData;
	while (!MessageQueue.IsEmpty())
	{
		FMemory::Memzero(CallbackData);
		if (SocketSubsystem && MessageQueue.Dequeue(CallbackData))
		{
			SocketSubsystem->SteamSocketEventHandler(&CallbackData);
		}
		else
		{
			break;
		}
	}
}
