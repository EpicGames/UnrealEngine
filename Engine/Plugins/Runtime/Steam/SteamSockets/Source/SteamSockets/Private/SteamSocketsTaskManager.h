// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SteamSocketsTaskManagerInterface.h"
#include "SteamSocketsPrivate.h" // IWYU pragma: keep
#include "Containers/Queue.h"

class FSteamSocketsSubsystem;

// A glorified event manager for handling status updates about SteamSockets we hold.
class FSteamSocketsTaskManager : public FSteamSocketsTaskManagerInterface
{
public:
	FSteamSocketsTaskManager(FSteamSocketsSubsystem* SocketSub) :
		FSteamSocketsTaskManagerInterface(SocketSub),
	    OnConnectionStatusChangedCallback(this, &FSteamSocketsTaskManager::OnConnectionStatusChanged),
		OnConnectionStatusChangedGSCallback(this, &FSteamSocketsTaskManager::OnConnectionStatusChangedGS)
	{
		
	}

	virtual ~FSteamSocketsTaskManager()
	{
		MessageQueue.Empty();
	}

	virtual void Tick() override;


private:
	STEAM_CALLBACK(FSteamSocketsTaskManager, OnConnectionStatusChanged, SteamNetConnectionStatusChangedCallback_t, OnConnectionStatusChangedCallback);
	STEAM_GAMESERVER_CALLBACK(FSteamSocketsTaskManager, OnConnectionStatusChangedGS, SteamNetConnectionStatusChangedCallback_t, OnConnectionStatusChangedGSCallback);

	// A copied queue of our messages we get from the SteamAPI. There's no difference between client and server messages
	// just how they are registered.
	TQueue<SteamNetConnectionStatusChangedCallback_t> MessageQueue;
};
