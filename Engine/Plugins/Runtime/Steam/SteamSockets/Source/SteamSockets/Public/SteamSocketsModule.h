// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"

#define UE_API STEAMSOCKETS_API

class FSteamSocketsModule : public IModuleInterface
{
public:

	FSteamSocketsModule() : bEnabled(false)
	{

	}

	virtual ~FSteamSocketsModule()
	{
	}

	// IModuleInterface

	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	static inline class FSteamSocketsModule& Get()
	{
		return FModuleManager::LoadModuleChecked<class FSteamSocketsModule>("SteamSockets");
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("SteamSockets");
	}
	
	bool IsSteamSocketsEnabled() const { return bEnabled; }

private:
	// Allows for easier configuration of which SteamSockets system to use.
	bool bEnabled;
};

#undef UE_API
