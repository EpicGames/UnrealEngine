// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/NetConnection.h"
#include "SteamSocketsNetConnection.generated.h"

#define UE_API STEAMSOCKETS_API

// Forward declare some types for functional type arguments
class FSocket;
class UNetDriver;
class FInternetAddr;
class FSteamSocket;

UCLASS(MinimalAPI, transient, config=Engine)
class USteamSocketsNetConnection : public UNetConnection
{
	GENERATED_BODY()

public:

	USteamSocketsNetConnection() :
		ConnectionSocket(nullptr),
		bInConnectionlessHandshake(false)
	{
	}

	//~ Begin NetConnection Interface
	UE_API virtual void CleanUp() override;
	UE_API virtual void InitBase(UNetDriver* InDriver, FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override;
	UE_API virtual void InitRemoteConnection(UNetDriver* InDriver, FSocket* InSocket, const FURL& InURL, const FInternetAddr& InRemoteAddr, 
		EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override;
	UE_API virtual void InitLocalConnection(UNetDriver* InDriver, FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override;
	UE_API virtual void LowLevelSend(void* Data, int32 CountBits, FOutPacketTraits& Traits) override;
	UE_API FString LowLevelGetRemoteAddress(bool bAppendPort=false) override;
	UE_API FString LowLevelDescribe() override;
	//~ End NetConnection Interface

private:
	const FSteamSocket* GetRawSocket() const { return ConnectionSocket; }

	UE_API void HandleRecvMessage(void* InData, int32 SizeOfData, const FInternetAddr* InFormattedAddress);

	void FlagForHandshake() { bInConnectionlessHandshake = true; }

	void ClearSocket() { ConnectionSocket = nullptr; }

	FSteamSocket* ConnectionSocket;
	bool bInConnectionlessHandshake;

	friend class USteamSocketsNetDriver;
};

#undef UE_API
