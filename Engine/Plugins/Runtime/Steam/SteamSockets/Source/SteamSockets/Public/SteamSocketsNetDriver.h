// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/NetDriver.h"
#include "SteamSocketsPackage.h"
#include "SteamSocketsTypes.h"
#include "SteamSocketsNetDriver.generated.h"

#define UE_API STEAMSOCKETS_API

class FNetworkNotify;

UCLASS(MinimalAPI, transient, config=Engine)
class USteamSocketsNetDriver : public UNetDriver
{
	GENERATED_BODY()

public:

	USteamSocketsNetDriver() :
		Socket(nullptr),
		bIsDelayedNetworkAccess(false)
	{
	}

	//~ Begin UObject Interface
	UE_API virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UNetDriver Interface.
	UE_API virtual void Shutdown() override;
	UE_API virtual bool IsAvailable() const override;
	UE_API virtual bool InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error) override;
	UE_API virtual bool InitConnect(FNetworkNotify* InNotify, const FURL& ConnectURL, FString& Error) override;
	UE_API virtual bool InitListen(FNetworkNotify* InNotify, FURL& LocalURL, bool bReuseAddressAndPort, FString& Error) override;
	UE_API virtual void TickDispatch(float DeltaTime) override;
	UE_API virtual void LowLevelSend(TSharedPtr<const FInternetAddr> Address, void* Data, int32 CountBits, FOutPacketTraits& Traits) override;
	UE_API virtual void LowLevelDestroy() override;
	UE_API virtual class ISocketSubsystem* GetSocketSubsystem() override;
	UE_API virtual bool IsNetResourceValid(void) override;
	//~ End UNetDriver Interface

	UE_API bool ArePacketHandlersDisabled() const;

protected:
	class FSteamSocket* Socket;
	bool bIsDelayedNetworkAccess;

	UE_API void ResetSocketInfo(const class FSteamSocket* RemovedSocket);

	UE_API UNetConnection* FindClientConnectionForHandle(SteamSocketHandles SocketHandle);

	UE_API void OnConnectionCreated(SteamSocketHandles ListenParentHandle, SteamSocketHandles SocketHandle);
	UE_API void OnConnectionUpdated(SteamSocketHandles SocketHandle, int32 NewState);
	UE_API void OnConnectionDisconnected(SteamSocketHandles SocketHandle);

	friend class FSteamSocketsSubsystem;
};

#undef UE_API
