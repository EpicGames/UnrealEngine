// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SocketSubsystem.h"
#include "IPAddress.h"
#include "Containers/Ticker.h"
#include "Containers/Map.h"
#include "Containers/Array.h"
#include "UObject/WeakObjectPtr.h"
#include "SteamSocketsPackage.h"
#include "SteamSocketsTaskManagerInterface.h"
#include "SteamSocketsTypes.h"
#include "SocketTypes.h"

#define UE_API STEAMSOCKETS_API

// Forward declare some internal types for bookkeeping.
class FSteamSocket;
class USteamSocketsNetDriver;

/**
 * Steam Sockets specific socket subsystem implementation. 
 * This class can only be used with the SteamSocketsNetDriver and the SteamSocketsNetConnection classes.
 * This subsystem does not support mixing any other NetDriver/NetConnection format. Doing so will cause this protocol to not function.
 */
class FSteamSocketsSubsystem : public ISocketSubsystem, public FTSTickerObjectBase, public FSelfRegisteringExec
{
public:

	FSteamSocketsSubsystem() :
		LastSocketError(0),
		bShouldTestPeek(false),
		SteamEventManager(nullptr),
		bUseRelays(true),
		SteamAPIClientHandle(nullptr),
		SteamAPIServerHandle(nullptr)
	{
	}

	//~ Begin SocketSubsystem Interface
	UE_API virtual bool Init(FString& Error) override;
	UE_API virtual void Shutdown() override;

	UE_API virtual class FSocket* CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolType) override;
	UE_API virtual void DestroySocket(class FSocket* Socket) override;

	UE_API virtual FAddressInfoResult GetAddressInfo(const TCHAR* HostName, const TCHAR* ServiceName = nullptr,
		EAddressInfoFlags QueryFlags = EAddressInfoFlags::Default,
		const FName ProtocolTypeName = NAME_None,
		ESocketType SocketType = ESocketType::SOCKTYPE_Unknown) override;
	UE_API virtual TSharedPtr<FInternetAddr> GetAddressFromString(const FString& IPAddress) override;

	UE_API virtual bool GetHostName(FString& HostName) override;

	UE_API virtual TSharedRef<FInternetAddr> CreateInternetAddr() override;
	UE_API virtual TSharedRef<FInternetAddr> CreateInternetAddr(const FName RequestedProtocol) override;

	UE_API virtual const TCHAR* GetSocketAPIName() const override;

	virtual ESocketErrors GetLastErrorCode() override {	return (ESocketErrors)LastSocketError; }
	virtual ESocketErrors TranslateErrorCode(int32 Code) override {	return (ESocketErrors)Code;	}

	UE_API virtual bool GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr>>& OutAddresses) override;
	UE_API virtual TArray<TSharedRef<FInternetAddr>> GetLocalBindAddresses() override;
	virtual bool HasNetworkDevice() override { return true; }
	virtual bool IsSocketWaitSupported() const override { return false; }
	virtual bool RequiresChatDataBeSeparate() override { return false; }
	virtual bool RequiresEncryptedPackets() override { return false; }
	//~ End SocketSubsystem Interface

	//~ Begin FTickerObject Interface
	UE_API virtual bool Tick(float DeltaTime) override;
	//~ End FTickerObject Interface

	/** Returns if the application is using the SteamSocket relays */
	bool IsUsingRelayNetwork() const { return bUseRelays; }

	/** Basic function to determine if Steam has been initialized properly. */
	bool IsSteamInitialized() const { return SteamAPIClientHandle.IsValid() || SteamAPIServerHandle.IsValid(); }

	static UE_API class ISteamNetworkingSockets* GetSteamSocketsInterface();

PACKAGE_SCOPE:
	/** A struct for holding steam socket information and managing bookkeeping on the protocol. */
	struct FSteamSocketInformation
	{
		FSteamSocketInformation(TSharedPtr<FInternetAddr> InAddr, FSteamSocket* InSocket, FSteamSocket* InParent = nullptr) :
			Addr(InAddr),
			Socket(InSocket),
			Parent(InParent),
			NetDriver(nullptr),
			bMarkedForDeletion(false)
		{
		}

		void MarkForDeletion();

		bool IsMarkedForDeletion() const { return bMarkedForDeletion; }

		bool operator==(const FSteamSocket* RHS) const
		{
			return Socket == RHS;
		}

		bool operator==(const FInternetAddr& InAddr) const;

		bool operator==(const FSteamSocketInformation& RHS) const
		{
			return RHS.Addr == Addr && RHS.Socket == Socket;
		}

		bool IsValid() const
		{
			return Addr.IsValid() && Parent != nullptr;
		}

		FString ToString() const;

		TSharedPtr<FInternetAddr> Addr;
		FSteamSocket* Socket;
		// Sockets created from a listener have a parent
		FSteamSocket* Parent;
		// The NetDriver for this connection.
		TWeakObjectPtr<USteamSocketsNetDriver> NetDriver;
	private:
		bool bMarkedForDeletion;
	};

	// Steam socket queriers
	UE_API FSteamSocketInformation* GetSocketInfo(SteamSocketHandles InternalSocketHandle);
	UE_API FSteamSocketInformation* GetSocketInfo(const FInternetAddr& ForAddress);

	// Steam socket bookkeeping modifiers
	UE_API void AddSocket(const FInternetAddr& ForAddr, FSteamSocket* NewSocket, FSteamSocket* ParentSocket = nullptr);
	UE_API void RemoveSocketsForListener(FSteamSocket* ListenerSocket);
	UE_API void QueueRemoval(SteamSocketHandles SocketHandle);
	UE_API void LinkNetDriver(FSocket* Socket, USteamSocketsNetDriver* NewNetDriver);

	// Delayed listen socket helpers.
	UE_API void AddDelayedListener(FSteamSocket* ListenSocket, USteamSocketsNetDriver* NewNetDriver);
	UE_API void OnServerLoginComplete(bool bWasSuccessful);

	// Returns this machine's identity in the form of a FInternetAddrSteamSockets
	UE_API TSharedPtr<FInternetAddr> GetIdentityAddress();
	
	// Returns if our account is currently logged into the Steam network
	UE_API bool IsLoggedInToSteam() const;

	/** Last error set by the socket subsystem or one of its sockets */
	int32 LastSocketError;

	// Singleton helpers
	static UE_API FSteamSocketsSubsystem* Create();
	static UE_API void Destroy();

	// SteamAPI internals handler
	UE_API void SteamSocketEventHandler(struct SteamNetConnectionStatusChangedCallback_t* ConnectionEvent);

	/** Flag for testing peek messaging (only usable in non-shipping builds) */
	bool bShouldTestPeek;

protected:
	//~ Begin FSelfRegisteringExec Interface
	UE_API virtual bool Exec_Dev(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	//~ End FSelfRegisteringExec Interface

	UE_API void CleanSocketInformation(bool bForceClean);
	UE_API void DumpSocketInformationMap() const;

	/** Single instantiation of this subsystem */
	static UE_API FSteamSocketsSubsystem* SocketSingleton;

	/** Event manager for Steam tasks */
	TUniquePtr<class FSteamSocketsTaskManagerInterface> SteamEventManager;

	/** Determines if the connections are going to be using the relay network */
	bool bUseRelays;

	/** Steam Client API Handle */
	TSharedPtr<class FSteamClientInstanceHandler> SteamAPIClientHandle;

	/** Steam Server API Handle */
	TSharedPtr<class FSteamServerInstanceHandler> SteamAPIServerHandle;

	/** Active connection bookkeeping */
	typedef TMap<SteamSocketHandles, FSteamSocketInformation> SocketHandleInfoMap;
	SocketHandleInfoMap SocketInformationMap;

	/** Structure for handling sockets that cannot be established due to platform login (for listener sockets) */
	struct FSteamPendingSocketInformation
	{
		FSteamSocket* Socket;
		TWeakObjectPtr<USteamSocketsNetDriver> NetDriver;

		FString ToString() const;
	};

	// Array of listeners we need to activate.
	TArray<FSteamPendingSocketInformation> PendingListenerArray;

	// Delegate handle for handling when a dedicated server logs into the Steam platform
	FDelegateHandle SteamServerLoginDelegateHandle;
};

#undef UE_API
