// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** 
 *  A base task manager interface for handling messages from the Steamworks SDK to pipe to the SteamSocketsSubsystem 
 *  This is mostly an empty interface used to allow for creation of uniqueptrs of the task manager interface
 *  without exposing Steam data types.
 */
class FSteamSocketsTaskManagerInterface
{
public:
	FSteamSocketsTaskManagerInterface(class FSteamSocketsSubsystem* SocketSub) :
		SocketSubsystem(SocketSub)
	{
	}
	
	virtual ~FSteamSocketsTaskManagerInterface() = default;

	/** A function that will be called on every update tick of the SteamSocketsSubsystem */
	virtual void Tick() = 0;

protected:
	class FSteamSocketsSubsystem* SocketSubsystem;
};
