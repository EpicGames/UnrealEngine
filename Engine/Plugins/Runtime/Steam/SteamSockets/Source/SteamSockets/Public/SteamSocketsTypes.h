// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"

#ifndef STEAM_SOCKETS_SUBSYSTEM
#define STEAM_SOCKETS_SUBSYSTEM FName(TEXT("SteamSockets"))
#endif

typedef uint32 SteamSocketHandles;
