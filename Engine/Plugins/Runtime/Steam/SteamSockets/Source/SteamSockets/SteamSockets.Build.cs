// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class SteamSockets : ModuleRules
{
    public SteamSockets(ReadOnlyTargetRules Target) : base(Target)
    {
		PublicDefinitions.Add("STEAMSOCKETS_MODULE=1");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core", 
				"CoreUObject",
				"NetCore",
				"Engine", 
				"Sockets",
				"OnlineSubsystem",
				"OnlineSubsystemSteam",
				"PacketHandler",
                "SteamShared"
            }
		);

		AddEngineThirdPartyPrivateStaticDependencies(Target, "Steamworks");
	}
}
