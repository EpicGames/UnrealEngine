{
	"FileVersion" : 3,
	"Version" : 1.0,
	"VersionName" : "1.0",
	"FriendlyName" : "Steam Sockets",
	"Description" : "New Steamworks Networking code that supports the SteamSockets interface. NOTE: This plugin is only compatible with the SteamSockets Netdriver. It will not work if the proper netdriver definitions have not been set.",
	"Category" : "Networking",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "SteamSockets",
			"Type" : "Runtime",
			"LoadingPhase" : "Default",
			"PlatformAllowList" :
			[
				"Win64",
				"Mac",
				"Linux"
			],
			"PlatformArchitectureDenyList" :
			[
				"Win64:arm64"
			]
		}
	],
	"Plugins": [
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		},
		{
			"Name": "OnlineSubsystemSteam",
			"Enabled": true
		},
		{
			"Name": "SteamShared",
			"Enabled": true
		}
	]
}